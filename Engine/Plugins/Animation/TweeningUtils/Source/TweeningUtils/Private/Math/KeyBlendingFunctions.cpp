// Copyright Epic Games, Inc. All Rights Reserved.

#include "Math/KeyBlendingFunctions.h"

#include "HAL/IConsoleManager.h"
#include "Math/Vector2D.h"

namespace UE::TweeningUtils
{
double Blend_ControlsToTween(double InBlendValue, const FVector2d& InBeforeBlendRange, const FVector2d& InAfterBlendRange)
{
	// Classic tween will move all to same location, not based on current time at all just blend and values
	const double NormalizedBlendValue = (InBlendValue + 1.0f) * 0.5f;
	const double Value = InBeforeBlendRange.Y + (InAfterBlendRange.Y - InBeforeBlendRange.Y) * (NormalizedBlendValue);
	return Value;
}

double Blend_PushPull(double InBlendValue, const FVector2d& InBeforeBlendRange, const FVector2d& InCurrent, const FVector2d& InAfterBlendRange)
{
	if (FMath::IsNearlyEqual(InAfterBlendRange.X, InBeforeBlendRange.X))
	{
		return InCurrent.Y;
	}
	const double T = (InCurrent.X - InBeforeBlendRange.X) / (InAfterBlendRange.X - InBeforeBlendRange.X);
	const double ValueAtT = InBeforeBlendRange.Y + T * (InAfterBlendRange.Y - InBeforeBlendRange.Y);
	double NewValue;
	if (InBlendValue < 0.0)
	{
		NewValue = InCurrent.Y + (-1.0 * InBlendValue) * (ValueAtT - InCurrent.Y);
	}
	else
	{
		const double AmplifyValueAtT = InCurrent.Y + (InCurrent.Y - ValueAtT);
		NewValue = InCurrent.Y + InBlendValue * (AmplifyValueAtT - InCurrent.Y);
	}
	return NewValue;
}

double Blend_Neighbor(double InBlendValue, const FVector2d& InBeforeBlendRange, const FVector2d& InCurrent, const FVector2d& InAfterBlendRange)
{
	double NewValue = InCurrent.Y;
	if (InBlendValue < 0.0)
	{
		NewValue = InCurrent.Y + (-1.0 * InBlendValue) * (InBeforeBlendRange.Y - InCurrent.Y);
	}
	else
	{
		NewValue = InCurrent.Y + InBlendValue * (InAfterBlendRange.Y - InCurrent.Y);
	}
	return NewValue;
}
	
double Blend_Relative(double InBlendValue, 
	const FVector2d& InBeforeBlendRange, const FVector2d& InFirstBlended,
	const FVector2d& InCurrent,
	const FVector2d& InLastBlended, const FVector2d& InAfterBlendRange
)
{
	double NewValue = InCurrent.Y;
	if (InBlendValue < 0.0)
	{
		NewValue = InCurrent.Y + (-1.0 * InBlendValue) * (InBeforeBlendRange.Y - InFirstBlended.Y);
	}
	else
	{
		NewValue = InCurrent.Y + InBlendValue * (InAfterBlendRange.Y - InLastBlended.Y);
	}
	return NewValue;
}

namespace Private::BlendToEase
{ 

static TAutoConsoleVariable<float> CVarEaseSlopeMultiplier(
	  TEXT("Tweening.Ease.SlopeMultiplier"),
	  6.0f,
	  TEXT("Controls the maximum slope of the Ease blend curve (at 100% slider). Higher = sharper peak curve.")
	  );

static TAutoConsoleVariable<float> CVarEaseSlopeExponent(
	TEXT("Tweening.Ease.SlopeExponent"),
	2.0f,
	TEXT("Controls how aggressively the Ease slope grows with slider position. "
		 "1.0 = linear, 2.0 = smoother behavior (gentler at low slider, sharper only at high slider).")
		 );


static float SCurve(float X, float Slope, float Width, float Height, float XShift, float YShift)
{
	if (X > (XShift + Width))
	{
		return(Height + YShift);
	}
	if (X < XShift)
	{
		return YShift;
	}

	const float Val = Height * (FMath::Pow((X - XShift), Slope) / (FMath::Pow(X - XShift, Slope) + FMath::Pow((Width - (X - XShift)), Slope))) + YShift;
	return Val;
}
}

double Blend_Ease(
	double InBlendValue, 
	const FVector2d& InBeforeBlendRange, const FVector2d& InCurrent, const FVector2d& InAfterBlendRange
	)
{
	if (FMath::IsNearlyEqual(InAfterBlendRange.X, InBeforeBlendRange.X))
	{
		return InCurrent.Y;
	}
	const double Source = InCurrent.Y;
	const double FullTimeDiff = InAfterBlendRange.X - InBeforeBlendRange.X;
	const double X = InCurrent.X - InBeforeBlendRange.X;
	const double Ratio = X / FullTimeDiff;
	double Shift, Delta, Base;
	if (InBlendValue > 0)
	{
		Shift = -1.0;
		Delta = InAfterBlendRange.Y - Source;
		Base = Source;
	}
	else
	{
		Shift = 0.0;
		Delta = Source - InBeforeBlendRange.Y;
		Base = InBeforeBlendRange.Y;
	}
	
	const float SlopeMultiplier = Private::BlendToEase::CVarEaseSlopeMultiplier.GetValueOnAnyThread();
	const float SlopeExponent   = Private::BlendToEase::CVarEaseSlopeExponent.GetValueOnAnyThread();
	const double AbsValue = FMath::Abs(InBlendValue);
	const double Slope = SlopeMultiplier * FMath::Pow(AbsValue, SlopeExponent);
	
	const double EaseY = Private::BlendToEase::SCurve(Ratio, Slope, 2.0, 2.0, Shift, Shift);
	const double NewValue = Base + (Delta * EaseY);

	return NewValue;
}

double Blend_SmoothRough(double InBlendValue, const FVector2d& InBeforeCurrent, const FVector2d& InCurrent, const FVector2d& InAfterCurrent)
{
	const double PrevVal = InBeforeCurrent.Y;
	const double CurVal = InCurrent.Y;
	const double NextVal = InAfterCurrent.Y;
	const double NewValue = (PrevVal * 0.25 + CurVal * 0.5 + NextVal * 0.25);
	if (InBlendValue < 0.0)
	{
		return InCurrent.Y + (-1.0 * InBlendValue) * (NewValue - InCurrent.Y);
	}
	else
	{
		return InCurrent.Y + InBlendValue * (InCurrent.Y - NewValue);
	}
}

double Blend_OffsetTime(
	double InBlendValue, const FVector2d& InCurrent,
	const FVector2d& InFirstBlended, const FVector2d& InLastBlended,
	const FVector2d& InBeforeBlendRange, const FVector2d& InAfterBlendRange,
	const TFunctionRef<double(double X)>& InEvaluate,
	bool bAllowExtrapolation
	)
{
	if (FMath::IsNearlyZero(InBlendValue))
	{
		return InCurrent.Y;
	}

	// A periodic function's period is the delta X at which y values start repeating. E.g. for sin(x), it's 2*PI.
	const double Period = InLastBlended.X - InFirstBlended.X;
	const double ShiftAmount = Period * InBlendValue;
	const double ShiftedX = InCurrent.X - ShiftAmount; // For function g to shift f to the right by a, g(x) := f(x - a).

	const auto BlendEdge = [&ShiftedX](const FVector2d& Border, const FVector2d& BeyondBorder)
	{
		const double ExceedAmount = ShiftedX - Border.X;
		const double BlendEdgeToNext = BeyondBorder.X - Border.X;
		const double BlendValue = FMath::IsNearlyZero(BlendEdgeToNext) ? 1.0 : FMath::Min(1.0, ExceedAmount / (BlendEdgeToNext));
		const double RemappedBlendValue = BlendValue * 2.0 - 1.0; // [0,1] to [-1, 1]
		return Blend_ControlsToTween(RemappedBlendValue, Border, BeyondBorder);
	};

	// If ShiftedX lies to the left or right of blended range of X values, use the Y value of the key to the left or right of the blended range, respectively.
	// Previous implementations would wrap around (i.e. evaluating it as mathematically periodic function) but that does not make any sense for animators.
	// When bAllowExtrapolation is true (e.g. cyclic curves), delegate to InEvaluate which handles the curve's infinity settings correctly.
	if (ShiftedX >= InLastBlended.X)
	{
		return bAllowExtrapolation ? InEvaluate(ShiftedX) : BlendEdge(InLastBlended, InAfterBlendRange);
	}
	if (ShiftedX <= InFirstBlended.X)
	{
		return bAllowExtrapolation ? InEvaluate(ShiftedX) : BlendEdge(InFirstBlended, InBeforeBlendRange);
	}
	
	return InEvaluate(ShiftedX);
}
}
