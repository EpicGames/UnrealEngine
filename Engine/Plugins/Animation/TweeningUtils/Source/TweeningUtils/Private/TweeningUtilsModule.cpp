// Copyright Epic Games, Inc. All Rights Reserved.

#include "TweeningUtilsModule.h"

namespace UE::TweeningUtils
{
void FTweeningUtilsModule::StartupModule()
{}

void FTweeningUtilsModule::ShutdownModule()
{}
	
IMPLEMENT_MODULE(FTweeningUtilsModule, TweeningUtils)
}
    