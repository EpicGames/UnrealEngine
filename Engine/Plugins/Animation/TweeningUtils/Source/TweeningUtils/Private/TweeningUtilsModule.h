// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"

namespace UE::TweeningUtils
{
class FTweeningUtilsModule : public IModuleInterface
{
public:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};
}
