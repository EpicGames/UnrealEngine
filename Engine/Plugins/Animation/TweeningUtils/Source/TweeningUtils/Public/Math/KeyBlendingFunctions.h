// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/MathFwd.h"
#include "Templates/FunctionFwd.h"

/**
 * The blending functions are generally used to iterate through a contiguous set of keys.
 * 
 * Visualized it looks like .xxxxxx., where '.'are keys that are not blended and 'x' keys that are blended.
 * There can be no gap between the keys, so '.xx.xxx.' is not a blendable range: it would need to be split up into two ranges: '.xx.' and '.xxx.'.
 * @see ContiguousKeyMapping.h
 * 
 * Generally the blending functions share the following arguments:
 *	- BlendValue		- The user specified value based on which to blend. Range is in [-1.0, 1.0]. 
 *	- BeforeBlendRange	- The key right before the range of blended keys. Same as FirstBlended if there is no such key.
 *	- FirstBlended		- The first key in the range of blended keys.
 *	- BeforeCurrent		- The key before the currently blended one. If Current is the first key, this is BeforeBlendRange.
 *	- Current			- The key currently being blended.
 *	- AfterCurrent		- The key after the current blended one. 
 *	- LastBlended		- The last key in the range of blended keys. If current is the last key, this is AfterBlendRange.
 *	- AfterBlendRange	- The key right after the range of blended keys. Same as LastBlended if there is no such key.
 * The blend functions return the new Y value that the current key should have.
 */
namespace UE::TweeningUtils
{
/**
 * Blend values of -1 or 1 move all keys to be linearly interpolated between the heights of BeforeBlendRange or AfterBlendRange, respectively.
 * @return The new Y value that the current key should have.
 */
TWEENINGUTILS_API double Blend_ControlsToTween(double InBlendValue, const FVector2d& InBeforeBlendRange, const FVector2d& InAfterBlendRange);
	
/**
 * Blend values of -1 or 1 flatten or exaggerates valleys and hills on the function, respectively.  
 * @return The new Y value that the current key should have.
 */
TWEENINGUTILS_API double Blend_PushPull(double InBlendValue,
	const FVector2d& InBeforeBlendRange, const FVector2d& InCurrent, const FVector2d& InAfterBlendRange
	);
	
/**
 * Blend values of -1 or 1 gradually interpolate keys to the height of the BeforeBlendRange or AfterBlendRange, respectively, using linear interpolation.
 *
 * Similar to Blend_Ease, which uses a S-curve instead.
 * Keys are moved up and down using linear interpolation - which comparatively moves much more suddenly.
 * 
 * @return The new Y value that the current key should have.
 */
TWEENINGUTILS_API double Blend_Neighbor(double InBlendValue,
	const FVector2d& InBeforeBlendRange, const FVector2d& InCurrent, const FVector2d& InAfterBlendRange
	);
	
/**
 * Blend values of -1 or 1 uniformly shift all keys down or up such that the left-most or right-most keys match up with the
 * key before or after the blend range, respectively.
 * 
 * @return The new Y value that the current key should have.
 */
TWEENINGUTILS_API double Blend_Relative(double InBlendValue, 
	const FVector2d& InBeforeBlendRange, const FVector2d& InFirstBlended,
	const FVector2d& InCurrent,
	const FVector2d& InLastBlended, const FVector2d& InAfterBlendRange
	);
	
/**
 * Blend values of -1 or 1 gradually interpolate keys to the height of the BeforeBlendRange or AfterBlendRange, respectively, using a smooth S curve.
 * @return The new Y value that the current key should have.
 */
TWEENINGUTILS_API double Blend_Ease(double InBlendValue, 
	const FVector2d& InBeforeBlendRange, const FVector2d& InCurrent, const FVector2d& InAfterBlendRange
	);

/**
 * Blend values of -1 or 1, push adjacent blended keys further together or apart, respectively. -1 averages keys out while 1 increases jumps.
 * Softens the curve or makes it harsh. Smooth is useful for softening noise, as found in mocap animations.
 * @return The new Y value that the current key should have.
 */
TWEENINGUTILS_API double Blend_SmoothRough(double InBlendValue, 
	const FVector2d& InBeforeCurrent, const FVector2d& InCurrent, const FVector2d& InAfterCurrent
	);

/**
 * Effectively shifts the curve to the left and right without actually changing the keys' X values: the Y values are recomputed to achieve the shift.
 * 
 * Blend values of -1 or 1 shift the range a period to the left or right, respectively; the period is in (InFirstBlendedX, InLastBlendedX).
 * For example, imagine the keys formed a sin wave: a blend value of 0.5 would effectively make it cos wave.
 * Keys for which the X value would be shifted out of the blend range have their Y value clamped.
 *
 * @param InBlendValue The blend value in [-1,1] that determines the shift amount (relative to InLastBlendedX - InFirstBlendedX).
 * @param InCurrent The value of the current key to blend
 * @param InFirstBlended The minimum of all keys being blended
 * @param InLastBlended The maximum X of all keys being blended
 * @param InBeforeBlendRange The value to return if the ShiftedX <= InFirstBlendedX.
 *	Usually the X of the first key before the blended range; should be InFirstBlendedX if there is none. 
 * @param InAfterBlendRange The value to return if the InLastBlendedX <= ShiftedX.
 *	Usually the X of the first key before the blended range; should be InLastBlendedX if there is none. 
 * @param InEvaluate Evaluates the function to shift; only needs to handle X in range InBlendInfo.FirstBlended <= X InBlendInfo.LastBlended.X,
 *	unless bAllowExtrapolation is true, in which case it must handle any X value.
 * @param bAllowExtrapolation If true, InEvaluate is called for ShiftedX values outside the blended range instead of clamping to the edge.
 *	Use this when the curve has cyclic extrapolation and InEvaluate already handles out-of-range X via the curve's infinity settings.
 *
 * @return The new Y value that the current key should have.
 */
TWEENINGUTILS_API double Blend_OffsetTime(
	double InBlendValue, const FVector2d& InCurrent,
	const FVector2d& InFirstBlended, const FVector2d& InLastBlended,
	const FVector2d& InBeforeBlendRange, const FVector2d& InAfterBlendRange,
	const TFunctionRef<double(double X)>& InEvaluate,
	bool bAllowExtrapolation = false
	);
}
