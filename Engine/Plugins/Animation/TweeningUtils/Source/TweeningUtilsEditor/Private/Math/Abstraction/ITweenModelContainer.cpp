// Copyright Epic Games, Inc. All Rights Reserved.

#include "Math/Abstraction/ITweenModelContainer.h"

#include "Containers/UnrealString.h"
#include "Misc/CoreMiscDefines.h"

namespace UE::TweeningUtilsEditor
{
int32 ITweenModelContainer::IndexOf(const FTweenModel& InTweenModel) const
{
	for (int32 Index = 0; Index < NumModels(); ++Index)
	{
		if (GetModel(Index) == &InTweenModel)
		{
			return Index;
		}
	}
	return INDEX_NONE;
}

FTweenModel* ITweenModelContainer::FindModelByIdentifier(const FString& InIdentifier)
{
	for (int32 Index = 0; Index < NumModels(); ++Index)
	{
		FTweenModel* Function = GetModel(Index);
		if (Function && GetModelIdentifier(*Function) == InIdentifier)
		{
			return Function;
		}
	}
	return nullptr;
}
}
