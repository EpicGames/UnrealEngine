// Copyright Epic Games, Inc. All Rights Reserved.

#include "Math/Abstraction/KeyBlendingAbstraction.h"

#include "Containers/UnrealString.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "Misc/Optional.h"
#include "Styling/SlateBrush.h"
#include "Templates/SharedPointer.h"
#include "TweeningUtilsCommands.h"
#include "TweeningUtilsStyle.h"

#define LOCTEXT_NAMESPACE "FStaticBlendFunctionData"

namespace UE::TweeningUtilsEditor
{
struct FStaticBlendFunctionData
{
	const EBlendFunction Type;
	const TWeakPtr<FUICommandInfo> Command; // Don't keep the command in a strong pointer case FTweeningUtilsCommands singleton gets destroyed
	const FText Label;
	const FText Abbreviation;
	const FText Description;
	const FString StringEncoding;
	const FString StyleBaseName;
	
	FStaticBlendFunctionData(
		EBlendFunction InType,
		TSharedPtr<FUICommandInfo> InCommand,
		FText InLabel,
		FText InAbbreviation,
		FText InDescription,
		FString InStringEncoding
		)
		: Type(InType)
		, Command(MoveTemp(InCommand))
		, Label(MoveTemp(InLabel))
		, Abbreviation(MoveTemp(InAbbreviation))
		, Description(MoveTemp(InDescription))
		, StringEncoding(MoveTemp(InStringEncoding))
		, StyleBaseName(FString::Printf(TEXT("TweeningUtils.%s"), InCommand ? *InCommand->GetCommandName().ToString() : TEXT("")))
	{
		check(InCommand.IsValid());
	}
	
	const FSlateBrush* GetUntintedIconForTweenFunction() const
	{
		const FString StyleString = StyleBaseName + TEXT(".Icon");
		return FTweeningUtilsStyle::Get().GetBrush(*StyleString);
	}

	FLinearColor GetTintColorForTweenFunction() const
	{
		const FString StyleString = StyleBaseName + TEXT(".Color");
		return FTweeningUtilsStyle::Get().GetColor(*StyleString);
	}
};

static FStaticBlendFunctionData& GetFunctionData(EBlendFunction InFunction)
{
	static_assert(static_cast<int32>(EBlendFunction::Num) == 7, "Extend this array");
	constexpr int32 Num = static_cast<int32>(EBlendFunction::Num);
	static FStaticBlendFunctionData Data[Num] =
	{
		FStaticBlendFunctionData(
			EBlendFunction::BlendNeighbor, FTweeningUtilsCommands::Get().SetTweenBlendNeighbor,
			LOCTEXT("BlendNeighbor.Label", "Blend Neighbor"),
			LOCTEXT("BlendNeighbour.Abbreviation", "BN"),
			LOCTEXT("BlendNeighbour.Description", "Blend to the next or previous values for selected keys."),
			TEXT("BlendNeighbour")
		),
		FStaticBlendFunctionData(
			EBlendFunction::PushPull, FTweeningUtilsCommands::Get().SetTweenPushPull,
			LOCTEXT("PushPull.Label", "Push / Pull"),
			LOCTEXT("PushPull.Abbreviation", "PP"),
			LOCTEXT("PushPull.Description", "Push or pull the values to the interpolation between the previous and next keys"),
			TEXT("PushPull")
		),
		FStaticBlendFunctionData(
			EBlendFunction::BlendEase, FTweeningUtilsCommands::Get().SetTweenBlendEase,
			LOCTEXT("BlendEase.Label", "Blend Ease"),
			LOCTEXT("BlendEase.Abbreviation", "BE"),
			LOCTEXT("BlendEase.Description", "Blend with an ease falloff to the next or previous value for selected keys"),
			TEXT("BlendEase")
		),
		FStaticBlendFunctionData(
			EBlendFunction::ControlsToTween, FTweeningUtilsCommands::Get().SetControlsToTween,
			LOCTEXT("ControlsToTween.Label", "Tween"),
			LOCTEXT("ControlsToTween.Abbreviation", "TW"),
			LOCTEXT("ControlsToTween.Description", "Interpolates between the previous and next keys"),
			TEXT("Tween")
		),
		FStaticBlendFunctionData(
			EBlendFunction::BlendRelative, FTweeningUtilsCommands::Get().SetTweenBlendRelative,
			LOCTEXT("BlendRelative.Label", "Move Relative"),
			LOCTEXT("BlendRelative.Abbreviation", "BR"),
			LOCTEXT("BlendRelative.Description", "Move relative to the next or previous value for selected keys"),
			TEXT("BlendRelative")
		),
		FStaticBlendFunctionData(
			EBlendFunction::SmoothRough, FTweeningUtilsCommands::Get().SetTweenSmoothRough,
			LOCTEXT("SmoothRough.Label", "Smooth Rough"),
			LOCTEXT("SmoothRough.Abbreviation", "SR"),
			LOCTEXT("SmoothRough.Description", "Push adjacent blended keys further together or apart. Smooth is useful for softening noise, like in mocap animations."),
			TEXT("SmoothRough")
		),
		FStaticBlendFunctionData(
			EBlendFunction::TimeOffset, FTweeningUtilsCommands::Get().SetTweenTimeOffset,
			LOCTEXT("TimeOffset.Label", "Time Offset"),
			LOCTEXT("TimeOffset.Abbreviation", "TO"),
			LOCTEXT("TimeOffset.Description", "Shifts the curve to the left / right by changing the keys' Y values and maintaining frame position."),
			TEXT("TimeOffset")
		)
	};

	const int32 Index = static_cast<int32>(InFunction);
	return Data[Index];
}

FString BlendFunctionToString(EBlendFunction InFunction)
{
	return GetFunctionData(InFunction).StringEncoding;
}

TOptional<EBlendFunction> LexBlendFunction(const FString& InString)
{
	for (int32 FuncIndex = 0; FuncIndex < static_cast<int32>(EBlendFunction::Num); ++FuncIndex)
	{
		const EBlendFunction Function = static_cast<EBlendFunction>(FuncIndex);
		if (InString == GetFunctionData(Function).StringEncoding)
		{
			return Function;
		}
	}
	return {};
}

TSharedPtr<FUICommandInfo> GetCommandForBlendFunction(EBlendFunction InFunction)
{
	const TSharedPtr<FUICommandInfo> Command = GetFunctionData(InFunction).Command.Pin();
	ensure(Command);
	return Command;
}

const FSlateBrush* GetUntintedIconForTweenFunction(EBlendFunction InFunction)
{
	return GetFunctionData(InFunction).GetUntintedIconForTweenFunction();
}

FLinearColor GetTintColorForTweenFunction(EBlendFunction InFunction)
{
	return GetFunctionData(InFunction).GetTintColorForTweenFunction();
}
	
FText GetLabelForBlendFunction(EBlendFunction InFunction)
{
	return GetFunctionData(InFunction).Label;
}

FText GetAbbreviationForBlendFunction(EBlendFunction InFunction)
{
	return GetFunctionData(InFunction).Abbreviation;
}

FText GetDescriptionForBlendFunction(EBlendFunction InFunction)
{
	return GetFunctionData(InFunction).Description;
}
}

#undef LOCTEXT_NAMESPACE