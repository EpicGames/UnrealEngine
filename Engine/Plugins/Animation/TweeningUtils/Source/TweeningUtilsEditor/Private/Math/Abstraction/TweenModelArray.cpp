// Copyright Epic Games, Inc. All Rights Reserved.

#include "Math/Abstraction/TweenModelArray.h"

#include "Containers/UnrealString.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Misc/Optional.h"
#include "Templates/Function.h"

namespace UE::TweeningUtilsEditor
{
void FTweenModelArray::ForEachModel(const TFunctionRef<void(FTweenModel&)> InConsumer) const
{
	for (const FTweenModelUIEntry& DisplayInfo : TweenModels)
	{
		const TUniquePtr<FTweenModel>& TweenModel = DisplayInfo.TweenModel;
		InConsumer(*TweenModel.Get());
	}
}

FTweenModel* FTweenModelArray::GetModel(int32 InIndex) const
{
	return TweenModels[InIndex].TweenModel.Get();
}

TSharedPtr<FUICommandInfo> FTweenModelArray::GetCommandForModel(const FTweenModel& InModel) const
{
	const int32 Index = IndexOf(InModel);
	return TweenModels.IsValidIndex(Index) ? TweenModels[Index].DisplayInfo.Command : nullptr;
}

const FSlateBrush* FTweenModelArray::GetIconForModel(const FTweenModel& InModel) const
{
	const int32 Index = IndexOf(InModel);
	return TweenModels.IsValidIndex(Index) ? TweenModels[Index].DisplayInfo.Brush : nullptr;
}

FLinearColor FTweenModelArray::GetColorForModel(const FTweenModel& InModel) const
{
	const int32 Index = IndexOf(InModel);
	return TweenModels.IsValidIndex(Index) ? TweenModels[Index].DisplayInfo.Color : FLinearColor::White;
}

FText FTweenModelArray::GetLabelForModel(const FTweenModel& InModel) const
{
	const int32 Index = IndexOf(InModel);
	return TweenModels.IsValidIndex(Index) ? TweenModels[Index].DisplayInfo.Label : FText::GetEmpty();
}
	
FText FTweenModelArray::GetToolTipForModel(const FTweenModel& InModel) const
{
	const int32 Index = IndexOf(InModel);
	return TweenModels.IsValidIndex(Index) ? TweenModels[Index].DisplayInfo.ToolTip : FText::GetEmpty();
}

FString FTweenModelArray::GetModelIdentifier(const FTweenModel& InModel) const
{
	const int32 Index = IndexOf(InModel);
	return TweenModels.IsValidIndex(Index) ? TweenModels[Index].DisplayInfo.Identifier : FString();
}
}
