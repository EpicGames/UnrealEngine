// Copyright Epic Games, Inc. All Rights Reserved.

#include "Math/ContiguousKeyMapping.h"

#include "Algo/BinarySearch.h"
#include "CurveDataAbstraction.h"
#include "CurveEditorTypes.h"
#include "CurveEditor.h"
#include "CurveEditorSelection.h"
#include "CurveModel.h"
#include "Math/NumericLimits.h"
#include "Misc/Optional.h"

namespace UE::TweeningUtilsEditor
{
static TOptional<int32> GetIndex(const TArray<FKeyPosition>& Keys, double Time)
{
	const int32 Index = Algo::LowerBoundBy(Keys, Time, [](const FKeyPosition& Value) { return Value.InputValue; });

	// don't trust precision issues so will double check to make sure the index is correct
	if (Keys.IsValidIndex(Index))
	{
		if (FMath::IsNearlyEqual(Keys[Index].InputValue, Time))
		{
			return Index;
		}
		if (((Index - 1) >= 0) && FMath::IsNearlyEqual(Keys[Index - 1].InputValue, Time))
		{
			return Index - 1;
		}
		if (((Index + 1) < Keys.Num()) && FMath::IsNearlyEqual(Keys[Index + 1].InputValue, Time))
		{
			return Index + 1;
		}
		return Index;
	}
	return {};
}

static void AppendKeyArray(
	const FCurveEditor& InCurveEditor,
	const FCurveModelID& InCurveId, TConstArrayView<FKeyHandle> KeysToBlend,
	TMap<FCurveModelID, FContiguousKeyMapping::FContiguousKeysArray>& OutKeyMap
	)
{
	if (const FCurveModel* Curve = InCurveEditor.FindCurve(InCurveId))
	{
		const TArray<FKeyHandle> AllKeyHandles = Curve->GetAllKeys();
		
		TArray<FKeyPosition> AllKeyPositions;
		AllKeyPositions.SetNum(AllKeyHandles.Num());
		Curve->GetKeyPositions(AllKeyHandles, AllKeyPositions);
		TArray<FVector2d> AllKeyPositionVectors;
		Algo::Transform(AllKeyPositions, AllKeyPositionVectors, [](const FKeyPosition& InKeyPostition)
		{
			return FVector2d{ InKeyPostition.InputValue, InKeyPostition.OutputValue };
		});

		// Get all the selected keys
		TArray<FKeyPosition> KeyPositions;
		KeyPositions.SetNum(KeysToBlend.Num());
		Curve->GetKeyPositions(KeysToBlend, KeyPositions);

		FContiguousKeyMapping::FContiguousKeysArray& KeyArray = OutKeyMap.Add(
			InCurveId, FContiguousKeyMapping::FContiguousKeysArray(AllKeyHandles, AllKeyPositionVectors)
			);

		TArray<int32> SelectedIndices;
		for (int32 Index = 0; Index < KeyPositions.Num(); ++Index)
		{
			const TOptional<int32> AllKeysIndex = GetIndex(AllKeyPositions, KeyPositions[Index].InputValue);
			if (AllKeysIndex)
			{
				SelectedIndices.Add(*AllKeysIndex);
			}
		}
		SelectedIndices.Sort();
		
		TArray<int32> ContiguousKeyIndices;
		for(int32 AllKeysIndex: SelectedIndices)
		{
			if (ContiguousKeyIndices.Num() > 0) //see if this key is next to the previous one
			{
				if (ContiguousKeyIndices[ContiguousKeyIndices.Num() - 1] + 1 == AllKeysIndex)
				{
					ContiguousKeyIndices.Add(AllKeysIndex);
				}
				else //not contiguous, so need to add to the OutKeyMap
				{
					//now start new set with the new index
					KeyArray.AddBlendRange(ContiguousKeyIndices);
					ContiguousKeyIndices.Reset();
					ContiguousKeyIndices.Add(AllKeysIndex);
				}
			}
			else//first key in this set so just add
			{
				ContiguousKeyIndices.Add(AllKeysIndex);
			}
		}
		if (ContiguousKeyIndices.Num() > 0)
		{
			KeyArray.AddBlendRange(ContiguousKeyIndices);
			ContiguousKeyIndices.Reset();
		}
	}
}

static TMap<FCurveModelID, FContiguousKeyMapping::FContiguousKeysArray> ComputeKeyMap(const FCurveEditor& InCurveEditor)
{
	const TMap<FCurveModelID, FKeyHandleSet>& SelectionKeyMap = InCurveEditor.Selection.GetAll();
	TMap<FCurveModelID, FContiguousKeyMapping::FContiguousKeysArray> KeyMap;
	for (const TPair<FCurveModelID, FKeyHandleSet>& Pair : SelectionKeyMap)
	{
		AppendKeyArray(InCurveEditor, Pair.Key, Pair.Value.AsArray(), KeyMap);
	}
	return KeyMap;
}

void FBlendRangesData::AddBlendRange(const TArray<int32>& InContiguousKeyIndices)
{
	if (AllKeyPositions.Num() > 0 && InContiguousKeyIndices.Num() > 0)
	{
		KeysArray.Add(FContiguousKeys(AllKeyPositions, InContiguousKeyIndices));
	}
}

void FContiguousKeyMapping::FContiguousKeysArray::AddBlendRange(const TArray<int32>& InContiguousKeyIndices)
{
	if (AllKeyHandles.Num() > 0 && AllKeyPositions.Num() == AllKeyHandles.Num() && InContiguousKeyIndices.Num() > 0)
	{
		KeysArray.Add(FContiguousKeys(AllKeyPositions, InContiguousKeyIndices));
	}
}

FContiguousKeyMapping::FContiguousKeyMapping(const FCurveEditor& InCurveEditor)
	: KeyMap(ComputeKeyMap(InCurveEditor))
{}

void FContiguousKeyMapping::Append(const FCurveEditor& InCurveEditor, const FCurveModelID& InCurveId, TConstArrayView<FKeyHandle> InKeysToBlend)
{
	AppendKeyArray(InCurveEditor, InCurveId, InKeysToBlend, KeyMap);
}
}
