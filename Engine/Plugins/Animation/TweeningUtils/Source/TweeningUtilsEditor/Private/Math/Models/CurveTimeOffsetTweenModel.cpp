// Copyright Epic Games, Inc. All Rights Reserved.

#include "Math/Models/CurveTimeOffsetTweenModel.h"

#include "SCurveEditorView.h"
#include "CurveModel.h"
#include "Math/CurveBlending.h"
#include "Math/KeyBlendingFunctions.h"

namespace UE::TweeningUtilsEditor
{
FCurveTimeOffsetTweenModel::FCurveTimeOffsetTweenModel(TAttribute<TWeakPtr<FCurveEditor>> InWeakCurveEditor)
	: WeakCurveEditor(MoveTemp(InWeakCurveEditor))
{}

void FCurveTimeOffsetTweenModel::StartBlendOperation()
{
	const TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Get().Pin();
	ContiguousKeySelection = CurveEditor ? FContiguousKeyMapping(*CurveEditor) : FContiguousKeyMapping{};

	OriginalBlendedCurves.Empty(OriginalBlendedCurves.Num());
	for (const TPair<FCurveModelID, FContiguousKeyMapping::FContiguousKeysArray>& Pair : ContiguousKeySelection.KeyMap)
	{
		const FCurveModelID CurveId = Pair.Key;
		const FCurveModel* Curve = CurveEditor->FindCurve(CurveId);

		// The curve model may not support CreateBufferedCurveCopy. In that case, it won't be blended. Too bad for it.
		TUniquePtr<IBufferedCurveModel> BufferCurveCopy = Curve ? Curve->CreateBufferedCurveCopy() : nullptr;
		if (BufferCurveCopy)
		{
			OriginalBlendedCurves.Add(CurveId, MoveTemp(BufferCurveCopy));
		}
	}
}

void FCurveTimeOffsetTweenModel::StopBlendOperation()
{
	ContiguousKeySelection.KeyMap.Empty(ContiguousKeySelection.KeyMap.Num());
	OriginalBlendedCurves.Empty(OriginalBlendedCurves.Num());
}

void FCurveTimeOffsetTweenModel::BlendValues(float InNormalizedValue)
{
	const TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Get().Pin();
	if (!CurveEditor)
	{
		return;
	}

	const float ScaledBlendValue = ScaleBlendValue(InNormalizedValue);
	TweeningUtilsEditor::BlendCurves_BySingleKey(*CurveEditor, ContiguousKeySelection,
		[this, ScaledBlendValue, &CurveEditor](
			const FCurveModelID& CurveId,
			const FContiguousKeyMapping::FContiguousKeysArray& AllBlendedKeys, const FContiguousKeys& CurrentBlendRange,
			int32 Index
		)
	{
		const FVector2d& CurrentKey = AllBlendedKeys.GetCurrent(CurrentBlendRange, Index);

		const double FallbackValue = CurrentKey.Y;
		const TUniquePtr<IBufferedCurveModel>* OriginalCurve = OriginalBlendedCurves.Find(CurveId);
		if (!OriginalCurve)
		{
			return FallbackValue;
		}

		const auto Evaluate = [&FallbackValue, &OriginalCurve](double X)
		{
			// Though it shouldn't fail normally, Evaluate can theoretically fail, in which case it does not write to OutputValue.
			// So, init OutputValue to a defined value.
			double OutputValue = FallbackValue;
			OriginalCurve->Get()->Evaluate(X, OutputValue);
			return OutputValue;
		};

		// For curves with cyclic extrapolation, allow the evaluate function to handle out-of-range X values
		// via the channel's infinity settings instead of clamping to the edge key value.
		bool bAllowExtrapolation = false;
		if (const FCurveModel* Curve = CurveEditor->FindCurve(CurveId))
		{
			FCurveAttributes Attributes;
			Curve->GetCurveAttributes(Attributes);
			bAllowExtrapolation =
				(Attributes.HasPreExtrapolation() && (Attributes.GetPreExtrapolation() == RCCE_Cycle || Attributes.GetPreExtrapolation() == RCCE_CycleWithOffset)) ||
				(Attributes.HasPostExtrapolation() && (Attributes.GetPostExtrapolation() == RCCE_Cycle || Attributes.GetPostExtrapolation() == RCCE_CycleWithOffset));
		}

		return TweeningUtils::Blend_OffsetTime(ScaledBlendValue, CurrentKey,
			AllBlendedKeys.GetFirstInBlendRange(CurrentBlendRange), AllBlendedKeys.GetLastInBlendRange(CurrentBlendRange),
			AllBlendedKeys.GetBeforeBlendRange(CurrentBlendRange), AllBlendedKeys.GetAfterBlendRange(CurrentBlendRange),
			Evaluate,
			bAllowExtrapolation
			);
	});
}
}
