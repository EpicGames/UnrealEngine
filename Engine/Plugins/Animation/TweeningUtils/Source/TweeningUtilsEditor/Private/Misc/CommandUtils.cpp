// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommandUtils.h"

#include "Framework/Application/IInputProcessor.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Input/Events.h"

namespace UE::TweeningUtilsEditor
{
namespace Private
{
/** Listens for a command's chord going up and executes a delegate. */
class FCommandChordUpInputProcessor : public IInputProcessor, public TSharedFromThis<FCommandChordUpInputProcessor>
{
	const TSharedRef<FUICommandInfo> Command;
	const FExecuteAction Action;

	bool IsDragCommandKeyUp(const FKeyEvent& InKeyEvent) const
	{
		bool bIsMovingSlider = false;
		
		for (uint32 i = 0; i < static_cast<uint8>(EMultipleKeyBindingIndex::NumChords); ++i)
		{
			EMultipleKeyBindingIndex ChordIndex = static_cast<EMultipleKeyBindingIndex>(i);
			const FInputChord& Chord = *Command->GetActiveChord(ChordIndex);
			bIsMovingSlider |= Chord.IsValidChord() && InKeyEvent.GetKey() == Chord.Key;
		}
		
		return bIsMovingSlider;
	}
	
public:

	FCommandChordUpInputProcessor(const TSharedRef<FUICommandInfo>& InCommand, FExecuteAction InAction) 
		: Command(InCommand)
		, Action(MoveTemp(InAction))
	{}
	
	//~ Begin IInputProcessor Interface
	virtual bool HandleKeyUpEvent(FSlateApplication&, const FKeyEvent& InKeyEvent) override
	{
		const bool bStop = IsDragCommandKeyUp(InKeyEvent);
		
		if (bStop)
		{
			FSlateApplication::Get().UnregisterInputPreProcessor(SharedThis(this));
			Action.Execute();
		}
		
		return bStop;
	}

	virtual void Tick(const float DeltaTime, FSlateApplication& SlateApp, TSharedRef<ICursor> Cursor) override {}
	//~ Begin IInputProcessor Interface
};

class FAutoRegisterInputProcessor : public FCommandChordUpListener
{
	const TSharedRef<IInputProcessor> Processor;
public:
	
	explicit FAutoRegisterInputProcessor(const TSharedRef<IInputProcessor>& InProcessor)
		: Processor(InProcessor)
	{
		FSlateApplication::Get().RegisterInputPreProcessor(Processor);
	}
	
	~FAutoRegisterInputProcessor()
	{
		// Calling this multiple is fine... it is fine if the processor deregistered itself.
		FSlateApplication::Get().UnregisterInputPreProcessor(Processor);
	}
};
}

TUniquePtr<FCommandChordUpListener> ListenForCommandChordUp(const TSharedRef<FUICommandInfo>& InCommand, FExecuteAction InAction)
{
	using namespace Private;
	return MakeUnique<FAutoRegisterInputProcessor>(
		MakeShared<FCommandChordUpInputProcessor>(InCommand, InAction)
		);
}

TUniquePtr<FCommandChordUpListener> ListenForCommandChordUp(const TSharedPtr<FUICommandInfo>& InCommand, FExecuteAction InAction)
{
	return ListenForCommandChordUp(InCommand.ToSharedRef(), InAction);
}
}

