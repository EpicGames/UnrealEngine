// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/UIAction.h"
#include "Templates/SharedPointerFwd.h"
#include "Templates/UniquePtr.h"

class FUICommandInfo;

namespace UE::TweeningUtilsEditor
{
class FCommandChordUpListener { public: virtual ~FCommandChordUpListener() = default; };

/**
 * Listens for the command chord going up as there is no built-in system for this.
 * @return Listener object. When it goes out of scope, the subscription is cleaned up.
 */
TUniquePtr<FCommandChordUpListener> ListenForCommandChordUp(const TSharedRef<FUICommandInfo>& InCommand, FExecuteAction InAction);
TUniquePtr<FCommandChordUpListener> ListenForCommandChordUp(const TSharedPtr<FUICommandInfo>& InCommand, FExecuteAction InAction);
}
