// Copyright Epic Games, Inc. All Rights Reserved.

#include "TweeningToolsUserSettings.h"

#include "UObject/UObjectGlobals.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TweeningToolsUserSettings)

UTweeningToolsUserSettings* UTweeningToolsUserSettings::Get()
{
	return GetMutableDefault<UTweeningToolsUserSettings>();
}
