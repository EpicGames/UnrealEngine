// Copyright Epic Games, Inc. All Rights Reserved.

#include "TweeningUtilsCommands.h"

#include "TweeningUtilsStyle.h"
#include "Internationalization/Internationalization.h"
#include "Math/Abstraction/KeyBlendingAbstraction.h"

#define LOCTEXT_NAMESPACE "FTweeningUtilsCommands"

namespace UE::TweeningUtilsEditor
{
FTweeningUtilsCommands::FTweeningUtilsCommands()
	: TCommands<FTweeningUtilsCommands>
	(
		TEXT("TweeningUtils"),
		NSLOCTEXT("Contexts", "TweeningUtils", "Tweening"),
		NAME_None,
		FTweeningUtilsStyle::Get().GetStyleSetName()
	)
{}

void FTweeningUtilsCommands::RegisterCommands()
{
	static_assert(static_cast<int32>(EBlendFunction::Num) == 7, "Add a command here");
	UI_COMMAND(SetControlsToTween, "Tween ", "Interpolates between the previous and next keys", EUserInterfaceActionType::RadioButton, FInputChord() );
	UI_COMMAND(SetTweenPushPull, "Push / Pull", "Push or pull the values to the interpolation between the previous and next keys", EUserInterfaceActionType::RadioButton, FInputChord() );
	UI_COMMAND(SetTweenBlendNeighbor, "Blend Neighbor", "Blend to the next or previous values for selected keys", EUserInterfaceActionType::RadioButton, FInputChord() );
	UI_COMMAND(SetTweenBlendRelative, "Move Relative", "Move relative to the next or previous value for selected keys", EUserInterfaceActionType::RadioButton, FInputChord() );
	UI_COMMAND(SetTweenBlendEase, "Blend Ease", "Blend with an ease falloff to the next or previous value for selected keys", EUserInterfaceActionType::RadioButton, FInputChord() );
	UI_COMMAND(SetTweenSmoothRough, "Smooth / Rough", "Push adjacent blended keys further together or apart. Smooth is useful for softening noise, like in mocap animations.", EUserInterfaceActionType::RadioButton, FInputChord() );
	UI_COMMAND(SetTweenTimeOffset, "Time Offset", "Shifts the curve to the left / right by changing the keys' Y values and maintaining frame position.", EUserInterfaceActionType::RadioButton, FInputChord() );
	UI_COMMAND(ToggleOvershootMode, "Overshoot mode", "Overshoot extends the available range over 100%", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::U, EModifierKey::Control) );
	
	UI_COMMAND(DragAnimSliderTool, "Drag Anim Slider Tool", "Drag the anim slider", EUserInterfaceActionType::Button, FInputChord(EKeys::U));
	UI_COMMAND(ChangeAnimSliderTool, "Change Anim Slider Tool", "Go to the next blend function", EUserInterfaceActionType::Button, FInputChord(EKeys::U, EModifierKey::Shift));
	
	UI_COMMAND(SetSliderLeft, "Set Slider left (100%)", "Drag the anim slider 100% to the left", EUserInterfaceActionType::Button, FInputChord(EKeys::Home));
	UI_COMMAND(SetSliderRight, "Set Slider right (100%)", "Drag the anim slider 100% to the right", EUserInterfaceActionType::Button, FInputChord(EKeys::End));
	UI_COMMAND(SetSliderHalfLeft, "Set Slider half left (50%)", "Drag the anim slider 50% to the left", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SetSliderHalfRight, "Set Slider half right (50%)", "Drag the anim slider 50% to the right", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SetSliderQuarterLeft, "Set Slider quarter left (25%)", "Drag the anim slider 25% to the left", EUserInterfaceActionType::Button, FInputChord(EKeys::Home, EModifierKey::Shift));
	UI_COMMAND(SetSliderQuarterRight, "Set Slider quarter right (25%)", "Drag the anim slider 25% to the right", EUserInterfaceActionType::Button, FInputChord(EKeys::End, EModifierKey::Shift));
}
}

#undef LOCTEXT_NAMESPACE
