// Copyright Epic Games, Inc. All Rights Reserved.

#include "TweeningUtilsEditorModule.h"

#include "TweeningUtilsCommands.h"
#include "TweeningUtilsStyle.h"

namespace UE::TweeningUtilsEditor
{
void FTweeningUtilsEditorModule::StartupModule()
{
	FTweeningUtilsCommands::Register();
	FTweeningUtilsStyle::Get();
}

void FTweeningUtilsEditorModule::ShutdownModule()
{
	FTweeningUtilsCommands::Unregister();
}
	
IMPLEMENT_MODULE(FTweeningUtilsEditorModule, TweeningUtilsEditor)
}

	