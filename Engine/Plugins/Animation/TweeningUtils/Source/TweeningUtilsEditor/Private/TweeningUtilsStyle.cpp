// Copyright Epic Games, Inc. All Rights Reserved.

#include "TweeningUtilsStyle.h"

#include "Math/Abstraction/KeyBlendingAbstraction.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Widgets/TweenSliderStyle.h"

namespace UE::TweeningUtilsEditor
{
FTweeningUtilsStyle& FTweeningUtilsStyle::Get()
{
	static FTweeningUtilsStyle Style;
	return Style;
}

FTweeningUtilsStyle::FTweeningUtilsStyle()
	: FSlateStyleSet("TweeningUtilsStyle")
{
	const FString PluginContentDir = FPaths::EnginePluginsDir() / TEXT("Animation") / TEXT("TweeningUtils") / TEXT("Resources");
	const FString EngineEditorSlateDir = FPaths::EngineContentDir() / TEXT("Editor") / TEXT("Slate");
	SetContentRoot(PluginContentDir);
	SetCoreContentRoot(EngineEditorSlateDir);
	
	const FVector2D Icon20x20(20.0f, 20.0f);
	
	// Tween - Slider colors
	static_assert(static_cast<int32>(EBlendFunction::Num) == 7, "Add a color here");
	const FLinearColor ControlsToTweenTint(FColor(247, 216, 43));
	const FLinearColor PushPullTint(FColor(95, 227, 103));
	const FLinearColor BlendNeighborTint(FColor(187, 107, 240));
	const FLinearColor BlendRelativeTint(FColor(134, 138, 253));
	const FLinearColor BlendEaseTint(FColor(225, 102, 182));
	const FLinearColor SmoothRoughTint(FColor(68,213, 191));
	const FLinearColor TimeOffsetTint(FColor(254, 133, 57));
	Set("TweeningUtils.SetControlsToTween.Color", ControlsToTweenTint);
	Set("TweeningUtils.SetTweenPushPull.Color", PushPullTint);
	Set("TweeningUtils.SetTweenBlendNeighbor.Color", BlendNeighborTint);
	Set("TweeningUtils.SetTweenBlendRelative.Color", BlendRelativeTint);
	Set("TweeningUtils.SetTweenBlendEase.Color", BlendEaseTint);
	Set("TweeningUtils.SetTweenSmoothRough.Color", SmoothRoughTint);
	Set("TweeningUtils.SetTweenTimeOffset.Color", TimeOffsetTint);
	
	
	// Tween - Commands
	static_assert(static_cast<int32>(EBlendFunction::Num) == 7, "Add command style");
	Set("TweeningUtils.SetControlsToTween", new IMAGE_BRUSH_SVG("Icons/CurveTween_16", Icon20x20, ControlsToTweenTint));
	Set("TweeningUtils.SetTweenPushPull", new IMAGE_BRUSH_SVG("Icons/CurvePushPull_16", Icon20x20, PushPullTint));
	Set("TweeningUtils.SetTweenBlendNeighbor", new IMAGE_BRUSH_SVG("Icons/CurveBlendNeighbor_16", Icon20x20, BlendNeighborTint));
	Set("TweeningUtils.SetTweenBlendRelative", new IMAGE_BRUSH_SVG("Icons/CurveMoveRelative_16", Icon20x20, BlendRelativeTint));
	Set("TweeningUtils.SetTweenBlendEase", new IMAGE_BRUSH_SVG("Icons/CurveBlendEase_16", Icon20x20, BlendEaseTint));
	Set("TweeningUtils.SetTweenSmoothRough", new IMAGE_BRUSH_SVG("Icons/CurveSmoothRough_16", Icon20x20, SmoothRoughTint));
	Set("TweeningUtils.SetTweenTimeOffset", new IMAGE_BRUSH_SVG("Icons/CurveTimeOffset_16", Icon20x20, TimeOffsetTint));
	Set("TweeningUtils.ToggleOvershootMode", new IMAGE_BRUSH_SVG("Icons/SliderOvershoot_20", Icon20x20));
	
	// Tween - Icons (no tint)
	static_assert(static_cast<int32>(EBlendFunction::Num) == 7, "Add untinted icon here");
	Set("TweeningUtils.SetControlsToTween.Icon", new IMAGE_BRUSH_SVG("Icons/CurveTween_16", Icon20x20));
	Set("TweeningUtils.SetTweenPushPull.Icon", new IMAGE_BRUSH_SVG("Icons/CurvePushPull_16", Icon20x20));
	Set("TweeningUtils.SetTweenBlendNeighbor.Icon", new IMAGE_BRUSH_SVG("Icons/CurveBlendNeighbor_16", Icon20x20));
	Set("TweeningUtils.SetTweenBlendRelative.Icon", new IMAGE_BRUSH_SVG("Icons/CurveMoveRelative_16", Icon20x20));
	Set("TweeningUtils.SetTweenBlendEase.Icon", new IMAGE_BRUSH_SVG("Icons/CurveBlendEase_16", Icon20x20));
	Set("TweeningUtils.SetTweenSmoothRough.Icon", new IMAGE_BRUSH_SVG("Icons/CurveSmoothRough_16", Icon20x20));
	Set("TweeningUtils.SetTweenTimeOffset.Icon", new IMAGE_BRUSH_SVG("Icons/CurveTimeOffset_16", Icon20x20));
	
	// Tween - Slider visuals
	Set("TweenSlider", FTweenSliderStyle());
	
	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FTweeningUtilsStyle::~FTweeningUtilsStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}
}
