// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/MVC/CycleFunctionController.h"

#include "Framework/Commands/UICommandInfo.h"
#include "Framework/Commands/UICommandList.h"
#include "Math/Abstraction/ITweenModelContainer.h"
#include "TweeningUtilsCommands.h"

namespace UE::TweeningUtilsEditor
{
FCycleFunctionController::FCycleFunctionController(
	TAttribute<const FTweenModel*> InCurrentTweenModelAttr,
	const TSharedRef<ITweenModelContainer>& InTweenModelContainer,
	FHandleTweenChange InHandleTweenChange,
	const TSharedRef<FUICommandList>& InCommandList,
	TSharedPtr<FUICommandInfo> InCycleCommand
	)
	: CurrentTweenModelAttr(MoveTemp(InCurrentTweenModelAttr))
	, TweenModelContainer(InTweenModelContainer)
	, HandleTweenChangeDelegate(MoveTemp(InHandleTweenChange))
	, CommandList(InCommandList)
	, CycleCommand(MoveTemp(InCycleCommand))
{
	check(CurrentTweenModelAttr.IsSet() || CurrentTweenModelAttr.IsBound());
	check(CycleCommand);
	CommandList->MapAction(CycleCommand, FExecuteAction::CreateRaw(this, &FCycleFunctionController::CycleToNextFunction));
}

FCycleFunctionController::FCycleFunctionController(
	TAttribute<const FTweenModel*> InCurrentTweenModelAttr,
	const TSharedRef<ITweenModelContainer>& InTweenModelContainer,
	FHandleTweenChange InHandleTweenChange,
	const TSharedRef<FUICommandList>& InCommandList
	)
	: FCycleFunctionController(
		MoveTemp(InCurrentTweenModelAttr), InTweenModelContainer, MoveTemp(InHandleTweenChange),
		InCommandList, FTweeningUtilsCommands::Get().ChangeAnimSliderTool
		)
{}

FCycleFunctionController::~FCycleFunctionController()
{
	CommandList->UnmapAction(CycleCommand);
}

void FCycleFunctionController::CycleToNextFunction() const
{
	const int32 Index = TweenModelContainer->IndexOf(*CurrentTweenModelAttr.Get());
	if (ensure(TweenModelContainer->IsValidIndex(Index)))
	{
		const int32 NextIndex = TweenModelContainer->IsValidIndex(Index + 1) ? Index + 1 : 0;
		HandleTweenChangeDelegate.Execute(*TweenModelContainer->GetModel(NextIndex));
		OnFunctionCycleCommandInvokedDelegate.Broadcast();
	}
}
}
