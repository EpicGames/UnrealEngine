// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/MVC/MouseSlidingController.h"

#include "Framework/Application/IInputProcessor.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/Commands/UICommandList.h"

#include "HAL/PlatformMisc.h"

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#endif

#if PLATFORM_MICROSOFT
// Not sure whether this is needed to use RECT since we're already including "HAL/PlatformMisc.h" but other code uses it, too...
#include "Microsoft/WindowsHWrapper.h"
#endif

namespace UE::TweeningUtilsEditor
{
/**
 * Orchestrates calls to StartSliding, StopSliding, and UpdateSliding
 * The command system does not expose: 1. detecting key up, and 2. mouse movement. This class handles that.
 * 
 * Created once FMouseSlidingController::DragSliderCommand has been invoked and destroyed once FMouseSlidingController::DragSliderCommand's key
 * bindings are released, which is detected by this class.
 *
 * FMouseSlidingController::DragSliderCommand is only invoked when the tab owning the command list, e.g. Curve Editor, is focused.
 * Until the user releases the key that triggers the interaction, it is fine to preprocess input - we'll act as if the tab continues to be focused
 * for the duration of the press.
 */
class FMouseSlidingInputProcessor : public IInputProcessor
{
	FMouseSlidingController& Owner;
	bool bIsLeftMouseButtonDown = false;
	bool bHasStopped = false;

	TOptional<FVector2D> LastMouse;

	bool IsDragCommandKeyUp(const FKeyEvent& InKeyEvent) const
	{
		bool bIsMovingSlider = false;
		
		for (uint32 i = 0; i < static_cast<uint8>(EMultipleKeyBindingIndex::NumChords); ++i)
		{
			EMultipleKeyBindingIndex ChordIndex = static_cast<EMultipleKeyBindingIndex>(i);
			const FInputChord& Chord = *Owner.DragSliderCommand->GetActiveChord(ChordIndex);
			bIsMovingSlider |= Chord.IsValidChord() && InKeyEvent.GetKey() == Chord.Key;
		}
		
		return bIsMovingSlider;
	}
	
public:

	FMouseSlidingInputProcessor(FMouseSlidingController& InOwner UE_LIFETIMEBOUND) : Owner(InOwner) {}
	
	//~ Begin IInputProcessor Interface
	virtual bool HandleKeyUpEvent(FSlateApplication&, const FKeyEvent& InKeyEvent) override
	{
		bHasStopped |= IsDragCommandKeyUp(InKeyEvent);
		return true;
	}

	virtual bool HandleMouseButtonDownEvent(FSlateApplication& SlateApp, const FPointerEvent& InMouseEvent) override
	{
		if (InMouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
		{
			bIsLeftMouseButtonDown = true;
			return true;
		}
		return false;
	}
	virtual bool HandleMouseButtonUpEvent(FSlateApplication& SlateApp, const FPointerEvent& InMouseEvent) override
	{
		if (InMouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
		{
			bIsLeftMouseButtonDown = false;
			return true;
		}
		return false;
	}

	virtual bool HandleMouseMoveEvent(FSlateApplication&, const FPointerEvent& MouseEvent) override
	{
		LastMouse = MouseEvent.GetScreenSpacePosition();
		// We handle the event; this prevents us from formally hovering any widgets for the duration of the operation.
		return bIsLeftMouseButtonDown;
	}
	
	virtual void Tick(const float, FSlateApplication&, TSharedRef<ICursor> InCursor) override
	{
		if (bHasStopped)
		{
			Owner.StopListeningForMouseEvents(*InCursor);
			return;
		}

		if (!bIsLeftMouseButtonDown && Owner.SlidingState->IsSliding())
		{
			Owner.StopSliding(*InCursor);
			return;
		}
		
		if (!LastMouse || !bIsLeftMouseButtonDown)
		{
			return;
		}

		const FVector2D Mouse = *LastMouse;
		LastMouse.Reset();
		
		if (bIsLeftMouseButtonDown && !Owner.SlidingState->IsSliding())
		{
			Owner.StartSliding(Mouse, *InCursor);
		}
		Owner.UpdateSliding(Mouse);
	}
	//~ Begin IInputProcessor Interface
};
	
FMouseSlidingController::FMouseSlidingController(
	TAttribute<float> InMaxSlideWidthAttr,
	const TSharedRef<FUICommandList>& InCommandList,
	TSharedPtr<FUICommandInfo> InDragSliderCommand
	)
	: MaxSlideWidthAttr(MoveTemp(InMaxSlideWidthAttr))
	, CommandList(InCommandList)
	, DragSliderCommand(MoveTemp(InDragSliderCommand))
{
	check(InMaxSlideWidthAttr.IsSet() || InMaxSlideWidthAttr.IsBound());
	check(DragSliderCommand);
	BindCommand();
}

FMouseSlidingController::~FMouseSlidingController()
{
	SlidingState.Reset();
	UnbindCommand();
}

void FMouseSlidingController::BindCommand()
{
	if (!CommandList->IsActionMapped(DragSliderCommand))
	{
		CommandList->MapAction(
			DragSliderCommand, FExecuteAction::CreateRaw(this, &FMouseSlidingController::StartListeningForMouseEvents)
		);
	}
}

void FMouseSlidingController::UnbindCommand() const
{
	if (CommandList->IsActionMapped(DragSliderCommand))
	{
		CommandList->UnmapAction(DragSliderCommand);
	}
}

FMouseSlidingController::FSlidingState::FSlidingState(FMouseSlidingController& InOwner)
	: InputProcessor(MakeShared<FMouseSlidingInputProcessor>(InOwner))
{
	FSlateApplication::Get().RegisterInputPreProcessor(InputProcessor);
}

FMouseSlidingController::FSlidingState::~FSlidingState()
{
	FSlateApplication::Get().UnregisterInputPreProcessor(InputProcessor);
}

void FMouseSlidingController::StartListeningForMouseEvents()
{
	// The user should first press U and then the LMB. If the LMB is pressed first, ignore.
	// Other tools, such as the selection marquee in Curve Editor, use it and the interaction of those tools is hard to get right otherwise.
	const bool bIsLeftMouseButtonDown = FSlateApplication::Get().GetPressedMouseButtons().Contains(EKeys::LeftMouseButton);
	if (bIsLeftMouseButtonDown)
	{
		return;
	}
	
	if (!SlidingState)
	{
		SlidingState.Emplace(*this);
	}
}

void FMouseSlidingController::StopListeningForMouseEvents(ICursor& InCursor)
{
	if (!SlidingState)
	{
		return;
	}

	const bool bWasSliding = SlidingState->IsSliding();
	if (bWasSliding)
	{
		StopSliding(InCursor);
	}

	SlidingState.Reset();
}

void FMouseSlidingController::StartSliding(const FVector2D& InInitialScreenLocation, ICursor& InCursor)
{
	SlidingState->InitialMouse = InInitialScreenLocation;
	
	const float SliderSize = MaxSlideWidthAttr.Get();
	const float SliderHalfSize = SliderSize / 2.0;
	
	RECT ClipRect;
	constexpr float LockHalfHeight = 12.f;
	ClipRect.left = FMath::RoundToInt(SlidingState->InitialMouse->X - SliderHalfSize);
	ClipRect.top = FMath::RoundToInt(SlidingState->InitialMouse->Y - LockHalfHeight);
	ClipRect.right = FMath::TruncToInt(SlidingState->InitialMouse->X + SliderHalfSize);
	ClipRect.bottom = FMath::TruncToInt(SlidingState->InitialMouse->Y + LockHalfHeight);
	// Lock the mouse to the size of the virtual slider area to give the user feedback when they've moved the mouse far enough to reach -1 or 1.
	InCursor.Lock(&ClipRect);

	OnStartSlidingDelegate.Broadcast();
}

void FMouseSlidingController::StopSliding(ICursor& InCursor)
{
	SlidingState->InitialMouse.Reset();
	InCursor.Lock(nullptr);
	OnStopSlidingDelegate.Broadcast();
}

void FMouseSlidingController::UpdateSliding(const FVector2D& InScreenLocation)
{
	check(SlidingState->InitialMouse);
	
	const float SliderSize = MaxSlideWidthAttr.Get();
	const float SliderHalfSize = SliderSize / 2.0;
	const float InitialX = SlidingState->InitialMouse->X;
	const float StartToMouse = InScreenLocation.X - InitialX;
	const float ClampedStartToMouse = FMath::Clamp(StartToMouse, -SliderHalfSize, SliderHalfSize);
	OnUpdateSlidingDelegate.Broadcast(ClampedStartToMouse / SliderHalfSize);
}
}
