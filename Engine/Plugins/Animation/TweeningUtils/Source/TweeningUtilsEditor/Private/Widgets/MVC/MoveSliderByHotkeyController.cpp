// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/MVC/MoveSliderByHotkeyController.h"

#include "Misc/CommandUtils.h"
#include "Templates/UniquePtr.h"
#include "TweeningUtilsCommands.h"
#include "Math/Models/TweenModel.h"

namespace UE::TweeningUtilsEditor
{
struct FMoveSliderByHotkeyController::FActiveCommandOverride
{
	const float SliderValueOverride;
	
	/** Cleans up the listener for the command chord going up. */
	const TUniquePtr<FCommandChordUpListener> ChordUpListener;

	explicit FActiveCommandOverride(float InSliderValueOverride, TUniquePtr<FCommandChordUpListener> ChordUpListener)
		: SliderValueOverride(InSliderValueOverride)
		, ChordUpListener(MoveTemp(ChordUpListener))
	{}
};

FMoveSliderByHotkeyController::FMoveSliderByHotkeyController(
	TAttribute<FTweenModel*> InTweenModelAttr, const TSharedRef<FUICommandList>& InCommandList
	)
	: TweenModelAttr(MoveTemp(InTweenModelAttr))
	, WeakCommandList(InCommandList)
{
	FTweeningUtilsCommands& Commands = FTweeningUtilsCommands::Get();
	const auto MapAction = [this, &InCommandList](const TSharedPtr<FUICommandInfo> Command, float SliderPosition)
	{
		InCommandList->MapAction(
			Command, 
			FExecuteAction::CreateRaw(this, &FMoveSliderByHotkeyController::HandleMoveSlider, SliderPosition, Command.ToWeakPtr())
			);
	};
	
	MapAction(Commands.SetSliderLeft, -1.f);
	MapAction(Commands.SetSliderRight, 1.f);
	MapAction(Commands.SetSliderHalfLeft, -0.5f);
	MapAction(Commands.SetSliderHalfRight, 0.5f);
	MapAction(Commands.SetSliderQuarterLeft, -0.25f);
	MapAction(Commands.SetSliderQuarterRight, 0.25f);
}

FMoveSliderByHotkeyController::~FMoveSliderByHotkeyController()
{
	if (const TSharedPtr<FUICommandList> CommandList = WeakCommandList.Pin(); CommandList && FTweeningUtilsCommands::IsRegistered())
	{
		FTweeningUtilsCommands& Commands = FTweeningUtilsCommands::Get();
		CommandList->UnmapAction(Commands.SetSliderLeft);
		CommandList->UnmapAction(Commands.SetSliderRight);
		CommandList->UnmapAction(Commands.SetSliderHalfLeft);
		CommandList->UnmapAction(Commands.SetSliderHalfRight);
		CommandList->UnmapAction(Commands.SetSliderQuarterLeft);
		CommandList->UnmapAction(Commands.SetSliderQuarterRight);
	}
}

TOptional<float> FMoveSliderByHotkeyController::GetCurrentSliderPosition() const
{
	return ActiveCommand ? ActiveCommand->SliderValueOverride : TOptional<float>();
}

void FMoveSliderByHotkeyController::HandleMoveSlider(float InBlendValue, TWeakPtr<FUICommandInfo> InCommandForChordUp)
{
	if (FTweenModel* TweenModel = TweenModelAttr.Get())
	{
		FExecuteAction HandleCommandChordUpAction = FExecuteAction::CreateRaw(this, &FMoveSliderByHotkeyController::HandleCommandChordUp);
		ActiveCommand = MakePimpl<FActiveCommandOverride>(
			InBlendValue, 
			ListenForCommandChordUp(InCommandForChordUp.Pin(), MoveTemp(HandleCommandChordUpAction))
			);
		
		TweenModel->BlendOneOff(InBlendValue);
	}
}

void FMoveSliderByHotkeyController::HandleCommandChordUp()
{
	// The point of this is to make the slider appear at the override position for as long as the command keys are pressed down... as visual feedback.
	ActiveCommand.Reset();
}
}
