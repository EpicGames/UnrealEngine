// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/MVC/STweenView.h"

#include "Math/Models/TweenModel.h"
#include "Styling/AppStyle.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/STweenSlider.h"

namespace UE::TweeningUtilsEditor
{
void STweenView::Construct(const FArguments& InArgs)
{
	check(InArgs._TweenModel.IsSet() || InArgs._TweenModel.IsBound());
	check(InArgs._SliderIcon.IsSet() || InArgs._SliderIcon.IsBound());
	check(InArgs._SliderColor.IsSet() || InArgs._SliderColor.IsBound());
	
	TweenModelAttr = InArgs._TweenModel;

	// Assuming that this is placed in a toolbar, we want the of normal and hovered color to be the same, i.e. bright as if hovered.
	SetForegroundColor(FAppStyle::GetSlateColor("CurveEditor.TweenForeground"));
	
	ChildSlot
	[
		SNew(SBox)
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center) 
		[
			SAssignNew(TweenSlider, STweenSlider)
			.SliderIcon(InArgs._SliderIcon)
			.SliderColor(InArgs._SliderColor)
			.OverrideSliderPosition(InArgs._OverrideSliderPosition)
			.ScaleRenderMode(this, &STweenView::GetBarRenderMode)
			.OnSliderDragStarted(this, &STweenView::OnDragStarted)
			.OnSliderDragEnded(this, &STweenView::OnDragEnded)
			.OnSliderValueDragged(this, &STweenView::OnDragValueUpdated)
			.OnPointValuePicked(this, &STweenView::OnPointPicked)
			.MapSliderValueToBlendValue(this, &STweenView::MapSliderValueToBlendValue)
		]
	];
}

ETweenScaleMode STweenView::GetBarRenderMode() const
{
	return TweenModelAttr.Get()->GetScaleMode();
}

void STweenView::OnDragStarted() const
{
	TweenModelAttr.Get()->StartBlendOperation();
}

void STweenView::OnDragEnded() const
{
	TweenModelAttr.Get()->StopBlendOperation();
}

void STweenView::OnDragValueUpdated(float Value) const
{
	TweenModelAttr.Get()->BlendValues(Value);
}

void STweenView::OnPointPicked(float Value) const
{
	TweenModelAttr.Get()->BlendOneOff(Value);
}

float STweenView::MapSliderValueToBlendValue(float Value) const
{
	return TweenModelAttr.Get()->ScaleBlendValue(Value);
}
}
