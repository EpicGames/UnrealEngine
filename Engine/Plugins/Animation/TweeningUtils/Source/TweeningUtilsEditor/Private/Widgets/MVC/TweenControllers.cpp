// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/MVC/TweenControllers.h"

#include "TweeningToolsUserSettings.h"
#include "Math/Abstraction/ITweenModelContainer.h"

namespace UE::TweeningUtilsEditor
{
namespace Private
{
static void SetupUserPreferredFunctionConfig(
	FTweenToolbarController& ToolbarController, const TSharedRef<ITweenModelContainer>& InTweenModels, FName UserPreferredFunctionContext
	)
{
	const bool bShouldUsePreferredFunctions = UserPreferredFunctionContext != NAME_None;
	if (!bShouldUsePreferredFunctions)
	{
		return;
	}

	// Restore currently selected function to the one that is in the config.
	const FString* PreferredFunctionId = UTweeningToolsUserSettings::Get()->GetPreferredTweenFunction(UserPreferredFunctionContext);
	FTweenModel* PreferredFunction = PreferredFunctionId ? InTweenModels->FindModelByIdentifier(*PreferredFunctionId) : nullptr;
	if (PreferredFunction)
	{
		ToolbarController.SetSelectedTweenModel(*PreferredFunction);
	}

	// Save the key function to config when the user changes it.
	ToolbarController.OnTweenFunctionChanged().AddLambda([InTweenModels, UserPreferredFunctionContext](const FTweenModel& TweenModel)
	{
		UTweeningToolsUserSettings::Get()->SetPreferredTweenFunction(
			UserPreferredFunctionContext, InTweenModels->GetModelIdentifier(TweenModel)
			);
	});
}
}
	
FTweenControllers::FTweenControllers(
	const TSharedRef<FUICommandList>& InCommandList,
	const TSharedRef<ITweenModelContainer>& InTweenModels,
	FName UserPreferredFunctionContext
)
	: ToolbarController(InCommandList, InTweenModels)
	, CycleFunctionController(
		ToolbarController.MakeSelectedConstTweenModelAttr(),
		 InTweenModels,
		FCycleFunctionController::FHandleTweenChange::CreateLambda([this](FTweenModel& TweenModel)
		{
			ToolbarController.SetSelectedTweenModel(TweenModel);
		}),
		InCommandList
		)
	, MouseSlidingController(ToolbarController.MakeSelectedTweenModelAttr(), InCommandList)
	, MoveSliderByHotkeyController(ToolbarController.MakeSelectedTweenModelAttr(), InCommandList)
{
	check(InTweenModels->NumModels() > 0);
	Private::SetupUserPreferredFunctionConfig(ToolbarController, InTweenModels, UserPreferredFunctionContext);
}

TOptional<float> FTweenControllers::GetCurrentSliderPosition() const
{
	if (const TOptional<float> MouseOverride = MouseSlidingController.GetCurrentSliderPosition())
	{
		return MouseOverride;
	}
	
	if (const TOptional<float> CommandOverride = MoveSliderByHotkeyController.GetCurrentSliderPosition())
	{
		return CommandOverride;
	}
	
	return {};
}
}
