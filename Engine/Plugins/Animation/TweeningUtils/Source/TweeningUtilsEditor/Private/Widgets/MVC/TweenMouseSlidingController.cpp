// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/MVC/TweenMouseSlidingController.h"

#include "HAL/IConsoleManager.h"
#include "Math/Models/TweenModel.h"
#include "Misc/Attribute.h"
#include "TweeningUtilsCommands.h"
#include "TweeningUtilsStyle.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/TweenSliderStyle.h"

namespace UE::TweeningUtilsEditor
{
static TAutoConsoleVariable<float> CVarOverrideTweenSliderWidth(
	TEXT("CurveEditor.SliderMouseWidth"),
	-1.f,
	TEXT("Specify positive value to override the sliding width. Non-positive will result in the default being used.")
	);
	
FTweenMouseSlidingController::FTweenMouseSlidingController(
	TAttribute<float> InMaxSlideWidthAttr,
	TAttribute<FTweenModel*> InTweenModelAttr,
	const TSharedRef<FUICommandList>& InCommandList,
	TSharedPtr<FUICommandInfo> InDragSliderCommand
	)
	: FMouseSlidingController(
		TAttribute<float>::CreateLambda([Attr = MoveTemp(InMaxSlideWidthAttr)]()
		{
			const float CVarOverride = CVarOverrideTweenSliderWidth.GetValueOnGameThread();
			return CVarOverride <= 0.f ? Attr.Get() : CVarOverride;
		}),
		InCommandList, MoveTemp(InDragSliderCommand)
		)
	, TweenModelAttr(MoveTemp(InTweenModelAttr))
{
	OnStartSliding().AddRaw(this, &FTweenMouseSlidingController::OnSliderStartMove);
	OnStopSliding().AddRaw(this, &FTweenMouseSlidingController::OnSliderEndMove);
	OnUpdateSliding().AddRaw(this, &FTweenMouseSlidingController::OnSliderMove);
}

FTweenMouseSlidingController::FTweenMouseSlidingController(TAttribute<FTweenModel*> InTweenModelAttr, const TSharedRef<FUICommandList>& InCommandList)
	: FTweenMouseSlidingController(
		[]
		{
			// This FTweenMouseSlidingController constructor overload is for convenience and assumes default values... API user can still use the
			// other overload if their icon size is different.
			constexpr float DefaultIconWidth = 20.f;
			
			const FTweenSliderStyle TweenStyle = FTweeningUtilsStyle::Get().GetWidgetStyle<FTweenSliderStyle>("TweenSlider");
			const float ButtonSize = TweenStyle.IconPadding.Left + DefaultIconWidth + TweenStyle.IconPadding.Right;
			// The bar is padded with half the button width on each side. So the total size is the bar plus the button brush size.
			const float WidgetWidth = TweenStyle.BarDimensions.X + ButtonSize;

			// The widget draw size needs to be multiplied by DPI to obtain the size the widget would have on screen.
			const TSharedPtr<SWindow> TopLevelWindow = FSlateApplication::Get().GetActiveTopLevelWindow();
			const float ScaleDPI = TopLevelWindow ? TopLevelWindow->GetDPIScaleFactor() : 1.f;
			return WidgetWidth * ScaleDPI;
		}(),
		MoveTemp(InTweenModelAttr),
		InCommandList,
		FTweeningUtilsCommands::Get().DragAnimSliderTool
	)
{}

void FTweenMouseSlidingController::OnSliderStartMove()
{
	TweenModelAttr.Get()->StartBlendOperation();
}

void FTweenMouseSlidingController::OnSliderEndMove()
{
	CurrentSliderPosition.Reset();
	TweenModelAttr.Get()->StopBlendOperation();
}
	
void FTweenMouseSlidingController::OnSliderMove(float InValue)
{
	CurrentSliderPosition = InValue;
	TweenModelAttr.Get()->BlendValues(InValue);
}
}
