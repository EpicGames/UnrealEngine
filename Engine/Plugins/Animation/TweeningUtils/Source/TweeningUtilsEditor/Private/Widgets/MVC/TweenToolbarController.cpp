// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/MVC/TweenToolbarController.h"

#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Math/Color.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "TweeningUtilsCommands.h"
#include "Math/Abstraction/ITweenModelContainer.h"
#include "Math/Models/TweenModel.h"
#include "Widgets/MVC/STweenView.h"

namespace UE::TweeningUtilsEditor
{
static const FName BaseOvershootBrushName(TEXT("OvershootMode"));
	
class FOvershootButtonStyleHack : public FSlateStyleSet
{
public:

	explicit FOvershootButtonStyleHack(const ITweenModelContainer& InTweenFunctions)
		: FSlateStyleSet(*FString::Printf(TEXT("TweeningUtils_OvershootOverride_%s"), *FGuid::NewGuid().ToString()))
	{
		const FString PluginContentDir = FPaths::EnginePluginsDir() / TEXT("Animation") / TEXT("TweeningUtils") / TEXT("Resources");
		const FString EngineEditorSlateDir = FPaths::EngineContentDir() / TEXT("Editor") / TEXT("Slate");
		FSlateStyleSet::SetContentRoot(PluginContentDir);
		FSlateStyleSet::SetCoreContentRoot(EngineEditorSlateDir);
		
		const FVector2D Icon20x20(20.0f, 20.0f);
		FSlateBrush* BaseBrush = new IMAGE_BRUSH_SVG("Icons/SliderOvershoot_20", Icon20x20);
		Set(BaseOvershootBrushName, BaseBrush);
		InTweenFunctions.ForEachModel([this, &InTweenFunctions, &BaseBrush](const FTweenModel& Function) 
		{
			FSlateBrush* Brush = new FSlateBrush(*BaseBrush);
			Brush->TintColor = InTweenFunctions.GetColorForModel(Function);
			Set(*GetOvershootButtonStyleName(InTweenFunctions, Function), Brush);
		});
		
		FSlateStyleRegistry::RegisterSlateStyle(*this);
	}

	virtual ~FOvershootButtonStyleHack() override
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*this);
	}

	static FString GetOvershootButtonStyleName(
		const ITweenModelContainer& InTweenFunctions, const FTweenModel& InFunction
		)
	{
		return InTweenFunctions.GetModelIdentifier(InFunction);
	}
};
	
FTweenToolbarController::FTweenToolbarController(
	const TSharedRef<FUICommandList>& InCommandList,
	const TSharedRef<ITweenModelContainer>& InTweenFunctions,
	int32 InInitialTweenModelIndex
	)
	: CommandList(InCommandList)
	, TweenModels(InTweenFunctions)
	, SelectedTweenModel(TweenModels->GetModel(InInitialTweenModelIndex))
	, OverrideStyle(MakePimpl<FOvershootButtonStyleHack>(*InTweenFunctions))
{
	// Function changing commands
	TweenModels->ForEachModel([this](FTweenModel& InTweenModel)
	{
		CommandList->MapAction(TweenModels->GetCommandForModel(InTweenModel),
			FUIAction(
			   FExecuteAction::CreateRaw(this, &FTweenToolbarController::SetTweenModel, &InTweenModel),
			   {},
			   FIsActionChecked::CreateRaw(this, &FTweenToolbarController::IsTweenModelSelected, &InTweenModel)
			)
		);
	});

	InCommandList->MapAction(FTweeningUtilsCommands::Get().ToggleOvershootMode,
		FUIAction(
			FExecuteAction::CreateRaw(this, &FTweenToolbarController::ToggleOvershootMode),
			FCanExecuteAction(),
			FIsActionChecked::CreateRaw(this, &FTweenToolbarController::IsOvershootModeEnabled)
			)
);
}

FTweenToolbarController::~FTweenToolbarController()
{
	// Function changing commands
	TweenModels->ForEachModel([this](const FTweenModel& InTweenModel)
	{
		CommandList->UnmapAction(TweenModels->GetCommandForModel(InTweenModel));
	});

	// If the command is invalid, we're shutting down and this doesn't need to be run.
	if (FTweeningUtilsCommands::IsRegistered())
	{
		CommandList->UnmapAction(FTweeningUtilsCommands::Get().ToggleOvershootMode);
	}
}

FTweenToolbarController::FAddToToolbarResult FTweenToolbarController::AddToToolbar(FToolBarBuilder& ToolBarBuilder, FMakeWidgetArgs InArgs) const
{
	FAddToToolbarResult Result;
	
	ToolBarBuilder.SetLabelVisibility(EVisibility::Visible);
	ToolBarBuilder.AddComboButton(
	   FUIAction(),
	   FOnGetContent::CreateRaw(this, &FTweenToolbarController::MakeTweenModeMenu),
	   TAttribute<FText>::CreateRaw(this, &FTweenToolbarController::GetLabelForComboBox),
	   TAttribute<FText>::CreateRaw(this, &FTweenToolbarController::GetToolTipForComboBox),
	   TAttribute<FSlateIcon>::CreateRaw(this, &FTweenToolbarController::GetIconForComboBox),
	   false, NAME_None, {}, {}, {}, EUserInterfaceActionType::Button, InArgs.FunctionSelectResizeParams
	   );
	ToolBarBuilder.SetLabelVisibility(EVisibility::Collapsed);

	const TSharedRef<STweenView> View = SNew(STweenView)
		.TweenModel(MakeSelectedTweenModelAttr())
		.SliderIcon(TAttribute<const FSlateBrush*>::CreateRaw(this, &FTweenToolbarController::GetIconForSlider))
		.SliderColor(TAttribute<FLinearColor>::CreateRaw(this, &FTweenToolbarController::GetColorForSlider))
		.OverrideSliderPosition(MoveTemp(InArgs.OverrideSliderPositionAttr));
	ToolBarBuilder.AddWidget(View, {}, NAME_None, true, {}, {}, InArgs.SliderResizeParams);
	Result.TweenSlider = View->GetTweenSlider();
	
	ToolBarBuilder.AddToolBarButton(FTweeningUtilsCommands::Get().ToggleOvershootMode,
		NAME_None, TAttribute<FText>(), TAttribute<FText>(),
		TAttribute<FSlateIcon>::CreateRaw(this, &FTweenToolbarController::GetOvershootModeIcon),
		NAME_None, {}, {}, {}, InArgs.OvershootResizeParams
		);

	return Result;
}

void FTweenToolbarController::SetSelectedTweenModel(FTweenModel& InTweenModel)
{
	if (ensure(TweenModels->Contains(InTweenModel)))
	{
		SetTweenModel(&InTweenModel);
	}
}

TSharedRef<SWidget> FTweenToolbarController::MakeTweenModeMenu() const
{
	FMenuBuilder MenuBuilder(false, CommandList);

	TweenModels->ForEachModel([this, &MenuBuilder](const FTweenModel& InTweenModel)
	{
		MenuBuilder.AddMenuEntry(TweenModels->GetCommandForModel(InTweenModel));
	});

	return MenuBuilder.MakeWidget();
}

FText FTweenToolbarController::GetLabelForComboBox() const
{
	return TweenModels->GetLabelForModel(*SelectedTweenModel);
}

FText FTweenToolbarController::GetToolTipForComboBox() const
{
	return TweenModels->GetToolTipForModel(*SelectedTweenModel);
}

FSlateIcon FTweenToolbarController::GetIconForComboBox() const
{
	return TweenModels->GetCommandForModel(*SelectedTweenModel)->GetIcon();
}

void FTweenToolbarController::SetTweenModel(FTweenModel* InTweenModel)
{
	SelectedTweenModel = InTweenModel;
	OnTweenFunctionChangedDelegate.Broadcast(*SelectedTweenModel);
}

bool FTweenToolbarController::IsTweenModelSelected(FTweenModel* InTweenModel) const
{
	return SelectedTweenModel == InTweenModel;
}

void FTweenToolbarController::ToggleOvershootMode() const
{
	TweenModels->ForEachModel([](FTweenModel& InTweenModel)
	{
		InTweenModel.SetScaleMode(
			InTweenModel.GetScaleMode() == ETweenScaleMode::Normalized ? ETweenScaleMode::Overshoot : ETweenScaleMode::Normalized
			);
	});

	OnOvershootModeCommandInvokedDelegate.Broadcast();
}

bool FTweenToolbarController::IsOvershootModeEnabled() const
{
	return SelectedTweenModel->GetScaleMode() == ETweenScaleMode::Overshoot;
}

FSlateIcon FTweenToolbarController::GetOvershootModeIcon() const
{
	return FSlateIcon(
		OverrideStyle->GetStyleSetName(),
		IsOvershootModeEnabled() ? BaseOvershootBrushName : FName(*OverrideStyle->GetOvershootButtonStyleName(*TweenModels, *SelectedTweenModel))
		);
}

const FSlateBrush* FTweenToolbarController::GetIconForSlider() const
{
	return TweenModels->GetIconForModel(*SelectedTweenModel);
}

FLinearColor FTweenToolbarController::GetColorForSlider() const
{
	return TweenModels->GetColorForModel(*SelectedTweenModel);
}
}
