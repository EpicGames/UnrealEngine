// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/STweenSlider.h"

#include "Framework/DelayedDrag.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Widgets/SToolTip.h"
#include "Widgets/TweenSliderDrawUtils.h"

#define LOCTEXT_NAMESPACE "STweenSlider"

namespace UE::TweeningUtilsEditor
{
class FScopedDisableThrottle : public FNoncopyable
{
public:

	FScopedDisableThrottle() { FSlateThrottleManager::Get().DisableThrottle(true); }
	~FScopedDisableThrottle(){ FSlateThrottleManager::Get().DisableThrottle(false); }
};
	
struct FSliderWidgetData
{
	/** Contains all values required for drawing the slider. */
	FTweenWidgetArgs WidgetArgs;

	/** Hover state as it was when the mouse was last moved. */
	FTweenSliderHoverState HoverState;
	/** Same length as the number of points in the slider. Corresponds to slider value, which is in [0,1] range. Updated when hover state is updated. */
	TArray<float> PointSliderValues;

	// Technically, we only need the pimpl to hide the above stuff but since we already have it,
	// we'll just put all impl data here so we're more flexible, avoid recompiles, make hotfixes easier, etc.
	
	struct FDelayedDragData
	{
		/** Used to detect drags. */
		FDelayedDrag DelayedDrag;

		/** Used to determine whether the slider value changes this tick. */
		float LastSliderPosition = 0.5f;

		/** Set when a drag is detected. Makes sure that dragging the slider updates the viewport in realtime if the slide is in a menu. */
		TOptional<FScopedDisableThrottle> ScopedPreventThrottle;
		
		explicit FDelayedDragData(const FVector2D& InInitialPosition, const FKey& InEffectiveKey)
			: DelayedDrag(InInitialPosition, InEffectiveKey)
		{}

		void EnterResponsiveMode() { ScopedPreventThrottle.Emplace(); }
	};
	/** Data valid while the mouse button is down. */
	TOptional<FDelayedDragData> DragData;
	/** Whether the mouse was on a point when pressed. */
	bool bStartedMouseDownOnPoint = false;

	/** The last mouse position the user had. */
	FVector2D LastMousePosition;
	/** The last slider position the user hovered the mouse with (regardless of whether dragging or not). */
	float LastMousePositionOnSliderBar = 0.5f;
	
	/**
	 * The position of the slider in range 0.0 (completely left) to 1.0 (completely right).
	 * This value is converted to -1 to 1 range for OnSliderChangedDelegate.
	 */
	float TargetSliderPosition = 0.5f;
	/** Interpolated value where the slider is actually drawn. This makes the slider feel smoother. */
	float AnimatedCurrentSliderPosition = 0.5;

	/** A custom tooltip is used so it can be marked as interactive while sliding the button (otherwise it would disappear). */
	TSharedPtr<SToolTip> ToolTip;
};

/** Converts [0,1] to [-1,1] */
static float AsymmetricToSymmetric(float InValue) { return InValue * 2.0 - 1.0; }
/** Converts [-1,1] to [0,1] */
static float SymmetricToAsymmetric(float InValue) { return (InValue + 1) / 2.0; }
	
static float ReconcileSliderPositionOverride(const FTweenWidgetArgs& InWidgetArgs, float InFallback, FTweenSliderDrawArgs& InArgs)
{
	const bool bHasOverride = InWidgetArgs.OverrideSliderPositionAttr.IsBound() || InWidgetArgs.OverrideSliderPositionAttr.IsSet();
	const TOptional<float> OverrideValue = bHasOverride ? InWidgetArgs.OverrideSliderPositionAttr.Get() : TOptional<float>();
	if (OverrideValue.IsSet())
	{
		// While overriding the slider value, something external is driving it... we want to draw its indicators as if it was being driven by mouse.
		InArgs.bDrawButtonPressed = true;
		InArgs.bIsDragging = true;
		InArgs.HoverState.bIsSliderHovered = true;
		return SymmetricToAsymmetric(*OverrideValue);
	}
	return InFallback;
}

	
void STweenSlider::Construct(const FArguments& InArgs)
{
	check(InArgs._Style);
	check(InArgs._ColorAndOpacity.IsSet() || InArgs._ColorAndOpacity.IsBound());
	checkf(InArgs._SliderIcon.IsSet() || InArgs._SliderIcon.IsBound(), TEXT("You must provide a getter - ok to return nullptr though"));
	check(InArgs._SliderColor.IsSet() || InArgs._SliderColor.IsBound());
	check(InArgs._ScaleRenderMode.IsSet() || InArgs._ScaleRenderMode.IsBound());
	
	Pimpl = MakePimpl<FSliderWidgetData>();
	
	FTweenWidgetArgs& WidgetArgs = Pimpl->WidgetArgs;
	WidgetArgs.Style = InArgs._Style;
	WidgetArgs.ColorAndOpacity = InArgs._ColorAndOpacity;
	WidgetArgs.SliderIconAttr = InArgs._SliderIcon;
	WidgetArgs.SliderColor = InArgs._SliderColor;
	WidgetArgs.ScaleRenderModeAttr = InArgs._ScaleRenderMode;
	WidgetArgs.OverrideSliderPositionAttr = InArgs._OverrideSliderPosition;

	Pimpl->ToolTip = SNew(SToolTip)
		.Text(this, &STweenSlider::GetToolTipText)
		// This prevents the tooltip from disappearing when the mouse is pressed down.
		.IsInteractive(this, &STweenSlider::IsMouseDown);

	OnSliderDragStartedDelegate = InArgs._OnSliderDragStarted;
	OnSliderDragStoppedDelegate = InArgs._OnSliderDragEnded;
	OnSliderChangedDelegate = InArgs._OnSliderValueDragged;
	OnPointValuePickedDelegate = InArgs._OnPointValuePicked;
	MapSliderValueToBlendValueDelegate = InArgs._MapSliderValueToBlendValue;
	if (!MapSliderValueToBlendValueDelegate.IsBound())
	{
		MapSliderValueToBlendValueDelegate = FMapSliderValueToBlendValue::CreateLambda([](double Value){ return Value; });
	}
}

FReply STweenSlider::OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	// User might click the bar background in quick succession to blend e.g. value 0.835 multiple times;
	// the clicks may be so quick that it triggers a double click.
	return OnMouseButtonDown(InMyGeometry, InMouseEvent);
}

FReply STweenSlider::OnMouseButtonDown(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent)
{
	if (InMouseEvent.GetEffectingButton() != EKeys::LeftMouseButton)
	{
		return FReply::Unhandled();
	}
	
	FTweenWidgetArgs& WidgetArgs = Pimpl->WidgetArgs;
	FGeometry SliderArea, IconArea;
	TArray<FGeometry> Points;
	GetSliderButtonGeometry(Pimpl->AnimatedCurrentSliderPosition, InGeometry, WidgetArgs, SliderArea, IconArea);
	GetPointHitTestGeometry(InGeometry, WidgetArgs, Points);
	const FTweenSliderHoverState HoverState = GetHoverState(InMouseEvent.GetScreenSpacePosition(), SliderArea, Points);

	const bool bIsHoveringPoint = HoverState.HoveredPointIndex.IsSet();
	Pimpl->bStartedMouseDownOnPoint = bIsHoveringPoint;
	if (bIsHoveringPoint)
	{
		// Originally, we had this func call in OnMouseButtonUp (to allow user to move the mouse during click)
		// but the delay (of user physically releasing mouse button) made it feel unresponsive.
		HandlePickValueOfCurrentlyHoveredPoint(InGeometry, InMouseEvent);
		return FReply::Handled().CaptureMouse(SharedThis(this)).PreventThrottling();
	}
	
	Pimpl->DragData.Emplace(InMouseEvent.GetScreenSpacePosition(), InMouseEvent.GetEffectingButton());
	
	// If the user clicks in the background (neither slider button nor points), just instantly move the slider there.
	if (!HoverState.bIsSliderHovered)
	{
		Pimpl->DragData->DelayedDrag.ForceDragStart();
		UpdateLastMousePositionOnSlider(InGeometry, InMouseEvent.GetScreenSpacePosition());
		Pimpl->TargetSliderPosition = Pimpl->LastMousePositionOnSliderBar;
		OnStartDragOp();
	}
	
	return FReply::Handled().CaptureMouse(SharedThis(this)).PreventThrottling();
}

FReply STweenSlider::OnMouseButtonUp(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent)
{
	if (InMouseEvent.GetEffectingButton() != EKeys::LeftMouseButton)
	{
		return FReply::Unhandled();
	}

	if (Pimpl->bStartedMouseDownOnPoint)
	{
		Pimpl->bStartedMouseDownOnPoint = false;
		return FReply::Handled().ReleaseMouseCapture();
	}
	
	if (Pimpl->DragData)
	{
		Pimpl->DragData.Reset();
		OnSliderDragStoppedDelegate.ExecuteIfBound();;
		
		Pimpl->AnimatedCurrentSliderPosition = 0.5f;
		Pimpl->TargetSliderPosition = 0.5f;
		
		return FReply::Handled()
			.ReleaseMouseCapture()
			.ReleaseMouseLock();
	}
	
	return FReply::Unhandled();
}

void STweenSlider::OnMouseLeave(const FPointerEvent& MouseEvent)
{
	Pimpl->HoverState = FTweenSliderHoverState{};
	Pimpl->bStartedMouseDownOnPoint = false;
}

FReply STweenSlider::OnMouseMove(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent)
{
	const FVector2D MousePosition = InMouseEvent.GetScreenSpacePosition();
	Pimpl->LastMousePosition = MousePosition;
	UpdateLastMousePositionOnSlider(InGeometry, MousePosition);
	
	if (Pimpl->DragData)
	{
		const bool bHasStarted = Pimpl->DragData->DelayedDrag.AttemptDragStart(InMouseEvent);
		if (bHasStarted)
		{
			OnStartDragOp();
		}

		if (IsDragging())
		{
			Pimpl->TargetSliderPosition = Pimpl->LastMousePositionOnSliderBar;
		}
		
		return FReply::Handled()
			.CaptureMouse(SharedThis(this))
			// Locking the mouse gives the user instant feedback about having reached the end of the slider range.
			.LockMouseToWidget(SharedThis(this))
			.PreventThrottling();
	}

	// Hover state needs to updated for OnPaint (don't update it in OnPaint because Slate shifts InGeometry in OnPaint, which messes up hovering computations)
	FTweenWidgetArgs& WidgetArgs = Pimpl->WidgetArgs;
	FGeometry SliderArea, IconArea;
	TArray<FGeometry> Points;
	GetSliderButtonGeometry(Pimpl->AnimatedCurrentSliderPosition, InGeometry, WidgetArgs, SliderArea, IconArea);
	GetPointHitTestGeometry(InGeometry, WidgetArgs, Points, Pimpl->PointSliderValues);
	Pimpl->HoverState = GetHoverState(MousePosition, SliderArea, Points);
	
	return Pimpl->bStartedMouseDownOnPoint
			? FReply::Handled().CaptureMouse(SharedThis(this))
			: FReply::Unhandled();
}

void STweenSlider::OnFinishedPointerInput()
{
	if (Pimpl->DragData
		&& Pimpl->DragData->DelayedDrag.IsDragging()
		&& Pimpl->DragData->LastSliderPosition != Pimpl->TargetSliderPosition)
	{
		Pimpl->DragData->LastSliderPosition = Pimpl->TargetSliderPosition;

		// OnSliderChangedDelegate may be an expensive operation!
		// We call OnSliderChangedDelegate here because OnMouseMove can be called 100s or 1000s of times per tick.
		// When OnFinishedPointerInput is called, we know the final mouse position for the frame.
		OnSliderChangedDelegate.ExecuteIfBound(
			// Convert [0,1] range to [-1,1] range.
			AsymmetricToSymmetric(Pimpl->TargetSliderPosition)
			);
	}
}

FVector2D STweenSlider::ComputeDesiredSize(float) const
{
	FTweenWidgetArgs& SliderStyle = Pimpl->WidgetArgs;
	const FSlateBrush* SliderIconBrush = SliderStyle.SliderIconAttr.Get();
	
	const FVector2D TotalButtonSize = SliderIconBrush
		? SliderIconBrush->ImageSize + SliderStyle.Style->IconPadding.GetDesiredSize()
		: FVector2D::ZeroVector;
	
	return FVector2D
	{
		// We want the button to fit at position 0 and 1 - so we need half the button's space each side.
		SliderStyle.Style->BarDimensions.X + TotalButtonSize.X, 
		FMath::Max(TotalButtonSize.Y, SliderStyle.Style->BarDimensions.Y)
	};
}
	
void STweenSlider::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	constexpr float InterpSpeed = 35.f;
	Pimpl->AnimatedCurrentSliderPosition = FMath::FInterpTo(
		Pimpl->AnimatedCurrentSliderPosition, Pimpl->TargetSliderPosition, InDeltaTime, InterpSpeed
		);
}

int32 STweenSlider::OnPaint(
	const FPaintArgs& InArgs,
	const FGeometry& InAllottedGeometry,
	const FSlateRect& InCullingRect,
	FSlateWindowElementList& OutDrawElements,
	int32 InLayerId,
	const FWidgetStyle& InWidgetStyle,
	bool bParentEnabled
	) const
{
	const bool bIsMouseButtonDown = IsMouseDown();
	FTweenWidgetArgs& WidgetArgs = Pimpl->WidgetArgs;
	FTweenSliderHoverState& HoverState = Pimpl->HoverState;
	
	TArray<float> NormalizedPositions;
	FTweenSliderDrawArgs DrawArgs;
	
	DrawArgs.bIsDragging = IsDragging();
	DrawArgs.bDrawButtonPressed = DrawArgs.bIsDragging
		|| (bIsMouseButtonDown && HoverState.bIsSliderHovered)
		|| (bIsMouseButtonDown && HoverState.HoveredPointIndex);
	DrawArgs.HoverState = HoverState;
	
	const float SliderPosition = DrawArgs.bDrawButtonPressed
		? Pimpl->AnimatedCurrentSliderPosition : ReconcileSliderPositionOverride(WidgetArgs, Pimpl->AnimatedCurrentSliderPosition, DrawArgs);
	
	GetBarGeometry(InAllottedGeometry, WidgetArgs, DrawArgs.BarArea);
	GetSliderButtonGeometry(
		SliderPosition, InAllottedGeometry, WidgetArgs, DrawArgs.SliderArea, DrawArgs.IconArea
		);
	GetDrawnPointGeometry(
		InAllottedGeometry, WidgetArgs, HoverState, bIsMouseButtonDown, DrawArgs.Points, DrawArgs.PointTypes, NormalizedPositions
		);
	
	if (DrawArgs.bIsDragging)
	{
		GetPassedPointStates(NormalizedPositions, SliderPosition, DrawArgs.PassedPoints);
		GetDragValueIndicationGeometry(InAllottedGeometry, WidgetArgs, SliderPosition, DrawArgs.DragValueIndication);
	}
	
	return DrawTweenSlider(DrawArgs, WidgetArgs, OutDrawElements, InLayerId, InWidgetStyle);
}

TSharedPtr<IToolTip> STweenSlider::GetToolTip()
{
	return Pimpl->ToolTip;
}

void STweenSlider::UpdateLastMousePositionOnSlider(const FGeometry& InGeometry, const FVector2D& InMousePos)
{
	// Compute the offset the widget has from the left side of the screen, subtracts that offset from mouse x, and divides mouse x by widget length.
	const float DistToLeftScreenEdge = InGeometry.GetAbsolutePosition().X;
	const float SliderAbsLength = InGeometry.GetAbsoluteSize().X;
	const float MouseInRange = FMath::Clamp(InMousePos.X - DistToLeftScreenEdge, 0, SliderAbsLength);
	Pimpl->LastMousePositionOnSliderBar = MouseInRange / SliderAbsLength;
}

bool STweenSlider::IsMouseDown() const
{
	return Pimpl->DragData.IsSet() || Pimpl->bStartedMouseDownOnPoint;
}

bool STweenSlider::IsDragging() const
{
	return Pimpl->DragData && Pimpl->DragData->DelayedDrag.IsDragging(); 
}

void STweenSlider::OnStartDragOp() const
{
	Pimpl->DragData->EnterResponsiveMode();
	OnSliderDragStartedDelegate.ExecuteIfBound();
}

void STweenSlider::HandlePickValueOfCurrentlyHoveredPoint(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent) const
{
	FTweenWidgetArgs& WidgetArgs = Pimpl->WidgetArgs;
	FGeometry SliderArea, IconArea;
	TArray<FGeometry> Points;
	TArray<float> PointValues;
	
	GetSliderButtonGeometry(Pimpl->AnimatedCurrentSliderPosition, InGeometry, WidgetArgs, SliderArea, IconArea);
	GetPointHitTestGeometry(InGeometry, WidgetArgs, Points, PointValues);
	const FTweenSliderHoverState HoverState = GetHoverState(InMouseEvent.GetScreenSpacePosition(), SliderArea, Points);

	const int32 HoverIndex = HoverState.HoveredPointIndex.Get(INDEX_NONE);
	if (PointValues.IsValidIndex(HoverIndex))
	{
		// TODO UE-282985: If this widget is in a menu, then Slate will throttle and prevent the viewport from updating.
		// Simply disable throttling for 1 frame does not work.
		OnPointValuePickedDelegate.ExecuteIfBound(AsymmetricToSymmetric(PointValues[HoverIndex]));
	}
}

FText STweenSlider::GetToolTipText() const
{
	const auto ConvertSliderValue = [this](float Value)
	{
		// While sliding the button, the mouse is locked to the widget space (see LockMouseToWidget use above).
		// However, the mouse is missing a couple pixels on the left and right to fully reach -1 and 1 (it reaches e.g. ~0.978).
		// Since we display the value in the tooltip, we want the value to be rounded: It'd be weird if you fully move the slider all the way,
		// and it does not say 1; effectively, we lie to the user, but they won't be able to tell the difference.
		constexpr float RoundThreshold = 0.03f;
		const bool bRoundLower = Value < RoundThreshold;
		const bool bRoundUpper = 1 - Value < RoundThreshold;
		const float AdjustedValue = bRoundLower ? 0.0 : bRoundUpper ? 1.0 : Value;
		
		return MapSliderValueToBlendValueDelegate.Execute(AsymmetricToSymmetric(AdjustedValue));
	};
	
	const int32 HoveredIndex = Pimpl->HoverState.HoveredPointIndex.Get(INDEX_NONE);
	const bool bIsHoveringPoint = Pimpl->PointSliderValues.IsValidIndex(HoveredIndex);
	if (bIsHoveringPoint)
	{
		return FText::Format(
			LOCTEXT("Tooltip.HoverPointFmt", "{0} blend value.\nClick to blend to this value."),
			ConvertSliderValue(Pimpl->PointSliderValues[HoveredIndex])
			);
	}

	if (Pimpl->HoverState.bIsSliderHovered && !IsDragging())
	{
		return LOCTEXT("Tooltip.DragSlider", "Drag to blend values.");
	}

	return FText::Format(
		LOCTEXT("Tooltip.CurrentBlendValue", "{0} blend value."),
		ConvertSliderValue(Pimpl->LastMousePositionOnSliderBar)
		);
}
}

#undef LOCTEXT_NAMESPACE