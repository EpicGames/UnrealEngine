// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/TweenSliderDrawUtils.h"

#include "Rendering/DrawElementTypes.h"
#include "Rendering/RenderingCommon.h"
#include "Widgets/ETweenScaleMode.h"

namespace UE::TweeningUtilsEditor
{
	
static FVector2D ComputeButtonSize(const FTweenWidgetArgs& InWidgetArgs)
{
	const FSlateBrush* SliderIconBrush = InWidgetArgs.SliderIconAttr.Get();
	const FVector2D IconSize = SliderIconBrush ? SliderIconBrush->ImageSize : FVector2D::ZeroVector;
	return IconSize + InWidgetArgs.Style->IconPadding.GetDesiredSize();
}

static FVector2D ComputeBarOffset(const FTweenWidgetArgs& InWidgetArgs)
{
	// The entire bar is shifted because we need enough space at position 0 to fit the bar
	const FVector2D HalfButtonOffset { ComputeButtonSize(InWidgetArgs).X / 2.0, 0.0 };
	return HalfButtonOffset;
}

static FVector2D GetBarDimensions(const FGeometry& AllottedGeometry, const FTweenWidgetArgs& InWidgetArgs)
{
	const FVector2D& Dimensions = InWidgetArgs.Style->BarDimensions;
	
	// This handles widgets slots set to Auto and Filled. At least take the amount of space defined by the style.
	// If there is more horizontal space available, fill that space out.
	// Do not fill vertically because it looks broken: use the style value.
	const FVector2D BarSize = FVector2D{ FMath::Max(Dimensions.X, AllottedGeometry.GetLocalSize().X), Dimensions.Y };
	
	return BarSize
		// Account for the offset (left) and inset (right); there needs to be enough space to fully drag the button to the left and right.
		- 2 * ComputeBarOffset(InWidgetArgs);
}
	
void GetBarGeometry(const FGeometry& AllottedGeometry, const FTweenWidgetArgs& InWidgetArgs, FGeometry& OutBarArea)
{
	const FVector2D BarSize = GetBarDimensions(AllottedGeometry, InWidgetArgs);
	OutBarArea = AllottedGeometry.MakeChild(
		BarSize, FSlateLayoutTransform(ComputeBarOffset(InWidgetArgs) + FVector2D{ 0.f, AllottedGeometry.Size.Y / 2.f - BarSize.Y / 2.f })
		);
}
	
static FGeometry MakeGeometryOnBar(
	float InNormalizedBarPos, const FTweenWidgetArgs& InWidgetArgs, const FVector2D& InSize, const FGeometry& InAllottedGeometry
	)
{
	const FVector2D Dimensions = GetBarDimensions(InAllottedGeometry, InWidgetArgs);
	const float HorizontalOffset = Dimensions.X * InNormalizedBarPos - (InSize.X / 2.f);
	return InAllottedGeometry.MakeChild(
		InSize,
			FSlateLayoutTransform(ComputeBarOffset(InWidgetArgs)
			// InAllottedGeometry.Size.Y / 2.f = vertical center of bar,
			// InSize.Y / 2.f = offset it needs to have from center of bar
			+ FVector2D{ HorizontalOffset, InAllottedGeometry.Size.Y / 2.f - InSize.Y / 2.f }
		)
	);
}
	
void GetDragValueIndicationGeometry(const FGeometry& InAllottedGeometry, const FTweenWidgetArgs& InWidgetArgs, float InSliderPosition, FGeometry& OutDragValueIndication)
{
	const FVector2D Dimensions = GetBarDimensions(InAllottedGeometry, InWidgetArgs);
	
	const float RelativeWidth = InSliderPosition - 0.5f;
	const float AbsoluteWidth = FMath::Abs(RelativeWidth) * Dimensions.X;
	const bool bHasDraggedLeft = InSliderPosition < 0.5f; 
	const float StartPos = bHasDraggedLeft ? InSliderPosition : 0.5f;

	const float HorizontalOffset = Dimensions.X * StartPos;
	const FVector2D Size { AbsoluteWidth , Dimensions.Y };
	OutDragValueIndication = InAllottedGeometry.MakeChild(
		Size,
		FSlateLayoutTransform(ComputeBarOffset(InWidgetArgs)
			// InAllottedGeometry.Size.Y / 2.f = vertical center of bar,
			// InSize.Y / 2.f = offset it needs to have from center of bar
			+ FVector2D(HorizontalOffset, InAllottedGeometry.Size.Y / 2.f - Size.Y / 2.f)
		)
	);
}

void GetSliderButtonGeometry(
	float InNormalizedPosition, const FGeometry& InAllottedGeometry, const FTweenWidgetArgs& InWidgetArgs, FGeometry& OutSliderArea, FGeometry& OutIconArea
	)
{
	OutSliderArea = MakeGeometryOnBar(InNormalizedPosition, InWidgetArgs, ComputeButtonSize(InWidgetArgs), InAllottedGeometry);

	const FSlateBrush* SliderIconBrush = InWidgetArgs.SliderIconAttr.Get();
	OutIconArea = SliderIconBrush
		? MakeGeometryOnBar(InNormalizedPosition, InWidgetArgs, SliderIconBrush->ImageSize, InAllottedGeometry)
		: InAllottedGeometry.MakeChild(FVector2D::ZeroVector, FSlateLayoutTransform(FVector2D::ZeroVector));
}
	
template<typename TCallback> requires std::is_invocable_v<TCallback, float/*Position*/, EPointType>
static void EnumeratePoints(const FTweenWidgetArgs& InWidgetArgs, TCallback&& AddPoint)
{
	AddPoint(0.f, EPointType::End); 
	AddPoint(1.f, EPointType::End); 

	if (InWidgetArgs.ScaleRenderModeAttr.Get() == ETweenScaleMode::Normalized)
	{
		// Range is -100% to 100%. Place Small points every 12.5%
		for (int32 PointIndex = 1; PointIndex < 8; ++PointIndex)
		{
			const float Percentage = PointIndex / 8.f;
			AddPoint(Percentage, EPointType::Small); 
		}
	}
	else
	{
		// Range is -200% to 200%. Place Small points every 25%
		for (int32 PointIndex = 1; PointIndex < 16; ++PointIndex)
		{
			const bool bShouldBeMediumPoint = PointIndex == 4 || PointIndex == 12;
			if (!bShouldBeMediumPoint)
			{
				const float Percentage = PointIndex / 16.0;
				AddPoint(Percentage, EPointType::Small); 
			}
		}
		
		AddPoint(0.25f, EPointType::Medium); 
		AddPoint(0.75f, EPointType::Medium); 
	}
	
	AddPoint(0.5f, EPointType::Medium); 
}
	
static const FTweenPointStyle* GetPointStyleFromPointType(const FTweenSliderStyle& InStyle, EPointType PointType)
{
	static_assert(static_cast<int32>(EPointType::Num) == 3, "Update this switch");
	switch (PointType)
	{
	case EPointType::Small: return &InStyle.SmallPoint;
	case EPointType::Medium: return &InStyle.MediumPoint;
	case EPointType::End: return &InStyle.EndPoint;
	default: checkNoEntry(); return nullptr;
	}
}
	
enum class EHoverState
{
	Normal,
	Hovered,
	Pressed
};
static FVector2D GetPointSize(EHoverState HoverState, const FTweenPointStyle& Style)
{
	switch (HoverState)
	{
	case EHoverState::Normal: return Style.Normal.ImageSize;
	case EHoverState::Hovered: return Style.Hovered.ImageSize;
	case EHoverState::Pressed: return Style.Pressed.ImageSize;
	default: checkNoEntry(); return FVector2d::ZeroVector;
	}
}
	
void GetDrawnPointGeometry(
	const FGeometry& InAllottedGeometry, const FTweenWidgetArgs& InWidgetArgs,
	const FTweenSliderHoverState& InHoverState, bool bIsMouseButtonDown,
	TArray<FGeometry>& OutPoints, TArray<EPointType>& OutPointTypes, TArray<float>& OutNormalizedPositions
	)
{
	OutPoints.Reset();
	OutPointTypes.Reset();
	OutNormalizedPositions.Reset();
	
	const int32 HoveredIndex = InHoverState.HoveredPointIndex.Get(INDEX_NONE);
	EnumeratePoints(InWidgetArgs,
		[&InAllottedGeometry, &InWidgetArgs, bIsMouseButtonDown, &OutPoints, &OutPointTypes, &OutNormalizedPositions, HoveredIndex]
		(float InPosition, EPointType PointType)
		{
			const int32 AddedPointIndex = OutPoints.Num();
			const EHoverState PointHoverState =  AddedPointIndex == HoveredIndex
				? bIsMouseButtonDown ? EHoverState::Hovered : EHoverState::Pressed
				: EHoverState::Normal;

			const FVector2D Size = GetPointSize(PointHoverState, *GetPointStyleFromPointType(*InWidgetArgs.Style, PointType));
			const FGeometry Geometry = MakeGeometryOnBar(
				InPosition, InWidgetArgs, Size, InAllottedGeometry
				);
			
			OutPoints.Add(Geometry);
			OutPointTypes.Add(PointType);
			OutNormalizedPositions.Add(InPosition);
		});
}
	
void GetPointHitTestGeometry(
	const FGeometry& InAllottedGeometry, const FTweenWidgetArgs& InWidgetArgs, TArray<FGeometry>& OutPoints, TArray<float>& OutPointSliderValues
	)
{
	OutPoints.Reset();
	OutPointSliderValues.Reset();
	
	EnumeratePoints(InWidgetArgs,
		[&InAllottedGeometry, &InWidgetArgs, &OutPoints, &OutPointSliderValues](float InPosition, EPointType PointType)
		{
			const FVector2D& HitTestSize = GetPointStyleFromPointType(*InWidgetArgs.Style, PointType)->HitTestSize;
			const FGeometry Geometry = MakeGeometryOnBar(
				InPosition, InWidgetArgs, HitTestSize, InAllottedGeometry
				);
			
			OutPoints.Add(Geometry);
			OutPointSliderValues.Add(InPosition);
		});
}

void GetPointHitTestGeometry(const FGeometry& InAllottedGeometry, const FTweenWidgetArgs& InWidgetArgs, TArray<FGeometry>& OutPoints)
{
	OutPoints.Reset();
	
	EnumeratePoints(InWidgetArgs,
		[&InAllottedGeometry, &InWidgetArgs, &OutPoints](float InPosition, EPointType PointType)
		{
			const FVector2D& HitTestSize = GetPointStyleFromPointType(*InWidgetArgs.Style, PointType)->HitTestSize;
			const FGeometry Geometry = MakeGeometryOnBar(
				InPosition, InWidgetArgs, HitTestSize, InAllottedGeometry
				);
			
			OutPoints.Add(Geometry);
		});
}

void GetPassedPointStates(const TArray<float>& InNormalizedPositions, float InSliderPosition, TBitArray<>& OutPassedPoints)
{
	const bool bIsDraggingLeft = InSliderPosition < 0.5f;
	for (const float Position : InNormalizedPositions)
	{
		const bool bIsLeft = Position < 0.5f;
		const bool bIsPassedPoint = (bIsDraggingLeft && bIsLeft && InSliderPosition < Position)
			|| (!bIsDraggingLeft && !bIsLeft && Position < InSliderPosition)
			// The center point is always passed.
			|| FMath::IsNearlyEqual(Position, 0.5f);
		OutPassedPoints.Add(bIsPassedPoint);
	}
}
	
FTweenSliderHoverState GetHoverState(const FVector2D& InMouseScreenSpace, const FGeometry& InButtonArea, const TArray<FGeometry>& InPoints)
{
	FTweenSliderHoverState Result;

	if (InButtonArea.IsUnderLocation(InMouseScreenSpace))
	{
		Result.bIsSliderHovered = true;
	}
	else
	{
		for (int32 PointIndex = 0; PointIndex < InPoints.Num(); ++PointIndex)
		{
			if (InPoints[PointIndex].IsUnderLocation(InMouseScreenSpace))
			{
				Result.HoveredPointIndex = PointIndex;
				break;
			}
		}
	}
	
	return Result;
}

static void DrawBar(
	const FTweenSliderDrawArgs& InDrawArgs,
	const FTweenWidgetArgs& InWidgetArgs,
	FSlateWindowElementList& OutDrawElements,
	int32 LayerId,
	const FWidgetStyle& InWidgetStyle
	)
{
	const FTweenSliderStyle* Style = InWidgetArgs.Style;
	
	const FSlateBrush* BarBrush = &Style->BarBrush;
	const FLinearColor FinalBackgroundColor = InWidgetArgs.ColorAndOpacity.Get().GetColor(InWidgetStyle)
		* BarBrush->TintColor.GetColor(InWidgetStyle)
		* InWidgetStyle.GetColorAndOpacityTint();
	FSlateDrawElement::MakeBox(
		OutDrawElements, LayerId, InDrawArgs.BarArea.ToPaintGeometry(), BarBrush, ESlateDrawEffect::None, FinalBackgroundColor
		);
}
	
static void DrawDragValueIndication(
	const FTweenSliderDrawArgs& InDrawArgs,
	const FTweenWidgetArgs& InWidgetArgs,
	FSlateWindowElementList& OutDrawElements,
	int32 LayerId,
	const FWidgetStyle& InWidgetStyle
	)
{
	if (!InDrawArgs.bIsDragging)
	{
		return;
	}

	const FTweenSliderStyle* Style = InWidgetArgs.Style;
	const FLinearColor BarColor =
		InWidgetArgs.ColorAndOpacity.Get().GetColor(InWidgetStyle)
		* InWidgetArgs.SliderColor.Get()
		* Style->PassedValueBackground.TintColor.GetColor(InWidgetStyle)
		* InWidgetStyle.GetColorAndOpacityTint();
	FSlateDrawElement::MakeBox(
		OutDrawElements, LayerId, InDrawArgs.DragValueIndication.ToPaintGeometry(), &Style->PassedValueBackground, ESlateDrawEffect::None, BarColor
		);
}

/** Should the point's color be multiplied with the color of the slider? */
enum class EUseSliderColor { Yes, No };
	
static TPair<const FSlateBrush*, EUseSliderColor> GetPointBrush(
	const FTweenSliderDrawArgs& InDrawArgs, const FTweenSliderStyle& Style, int32 Index
	)
{
	const FTweenPointStyle* PointStyle = GetPointStyleFromPointType(Style, InDrawArgs.PointTypes[Index]);
	
	const bool bIsPassedPoint = InDrawArgs.PassedPoints.IsValidIndex(Index) && InDrawArgs.PassedPoints[Index];
	if (bIsPassedPoint)
	{
		// Passed points are white
		return { &PointStyle->PassedPoint, EUseSliderColor::No };
	}
	
	if (InDrawArgs.bIsDragging || InDrawArgs.HoverState.HoveredPointIndex != Index)
	{
		return { &PointStyle->Normal , EUseSliderColor::Yes };
	}
	
	if (InDrawArgs.bDrawButtonPressed)
	{
		// Pressed points are white
		return { &PointStyle->Pressed, EUseSliderColor::No };
	}
	return { &PointStyle->Hovered, EUseSliderColor::Yes };
}

static void DrawBarPoints(
	const FTweenSliderDrawArgs& InDrawArgs,
	const FTweenWidgetArgs& InWidgetArgs,
	FSlateWindowElementList& OutDrawElements,
	int32 LayerId,
	const FWidgetStyle& InWidgetStyle
	)
{
	const FTweenSliderStyle* Style = InWidgetArgs.Style;
	const FLinearColor ColorAndOpacity = InWidgetArgs.ColorAndOpacity.Get().GetColor(InWidgetStyle);
	const FLinearColor SliderColor = InWidgetArgs.SliderColor.Get();
	
	for (int32 PointIndex = 0; PointIndex < InDrawArgs.Points.Num(); ++PointIndex)
	{
		const FGeometry& PointGeometry = InDrawArgs.Points[PointIndex];
		const auto[Brush, UseSliderColor] = GetPointBrush(InDrawArgs, *Style, PointIndex);
		
		const FLinearColor PointColor = UseSliderColor == EUseSliderColor::Yes
			? ColorAndOpacity * SliderColor * Brush->TintColor.GetColor(InWidgetStyle)
			: ColorAndOpacity * Brush->TintColor.GetColor(InWidgetStyle)
			* InWidgetStyle.GetColorAndOpacityTint();
		FSlateDrawElement::MakeBox(
			OutDrawElements, LayerId, PointGeometry.ToPaintGeometry(), Brush, ESlateDrawEffect::None, PointColor
		);
	}
}

static void DrawButton(
	const FTweenSliderDrawArgs& InDrawArgs,
	const FTweenWidgetArgs& InWidgetArgs,
	FSlateWindowElementList& OutDrawElements,
	int32 LayerId,
	const FWidgetStyle& InWidgetStyle
	)
{
	const FTweenSliderStyle* Style = InWidgetArgs.Style;
	
	const FSlateBrush* SliderIconBrush = InWidgetArgs.SliderIconAttr.Get();
	const FSlateBrush* SliderButtonBrush = InDrawArgs.bDrawButtonPressed
		? &Style->PressedSliderButton
		: InDrawArgs.HoverState.bIsSliderHovered ? &Style->HoveredSliderButton : &Style->NormalSliderButton;

	const FSlateColor& BaseIconTint = InDrawArgs.bDrawButtonPressed
		? Style->PressedIconTint
		: InDrawArgs.HoverState.bIsSliderHovered ? Style->HoveredIconTint : Style->NormalIconTint;
	
	const FLinearColor ColorAndOpacity = InWidgetArgs.ColorAndOpacity.Get().GetColor(InWidgetStyle);
	const FLinearColor ButtonTint = ColorAndOpacity
		* InWidgetArgs.SliderColor.Get()
		* SliderButtonBrush->TintColor.GetColor(InWidgetStyle) 
		* InWidgetStyle.GetColorAndOpacityTint();;
	
	FSlateDrawElement::MakeBox(
		OutDrawElements, LayerId, InDrawArgs.SliderArea.ToPaintGeometry(), SliderButtonBrush, ESlateDrawEffect::None, ButtonTint
		);
	if (SliderIconBrush)
	{
		const FLinearColor IconTint = ColorAndOpacity
			* BaseIconTint.GetColor(InWidgetStyle)
			* SliderIconBrush->TintColor.GetColor(InWidgetStyle)
			* InWidgetStyle.GetColorAndOpacityTint();
		FSlateDrawElement::MakeBox(
			OutDrawElements, LayerId + 1, InDrawArgs.IconArea.ToPaintGeometry(), SliderIconBrush, ESlateDrawEffect::None, IconTint
		);
	}
}

int32 DrawTweenSlider(
	const FTweenSliderDrawArgs& InDrawArgs,
	const FTweenWidgetArgs& InWidgetArgs,
	FSlateWindowElementList& OutDrawElements,
	int32 LayerId,
	const FWidgetStyle& InWidgetStyle
	)
{
	DrawBar(InDrawArgs, InWidgetArgs, OutDrawElements, LayerId, InWidgetStyle);
	DrawDragValueIndication(InDrawArgs, InWidgetArgs, OutDrawElements, LayerId, InWidgetStyle);
	DrawBarPoints(InDrawArgs, InWidgetArgs, OutDrawElements, LayerId, InWidgetStyle);
	DrawButton(InDrawArgs, InWidgetArgs, OutDrawElements, LayerId, InWidgetStyle);
	
	return LayerId;
}
}
