// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "HAL/Platform.h"
#include "Layout/Geometry.h"
#include "Math/Color.h"
#include "Misc/Attribute.h"
#include "Styling/SlateColor.h"
#include "Widgets/TweenSliderStyle.h"

class FSlateWindowElementList;
class FWidgetStyle;
namespace UE::TweeningUtilsEditor { enum class ETweenScaleMode : uint8; }

namespace UE::TweeningUtilsEditor
{	
struct FTweenSliderHoverState
{
	/** Whether the slider button is pressed. */
	bool bIsSliderHovered = false;

	/** Index of FTweenSliderDrawArgs::Points that is hovered. */
	TOptional<int32> HoveredPointIndex;
};

enum class EPointType : uint8
{
	Small,
	/** When overshoot mode is enabled, a bar at 100% and -100% */
	Medium,
	/** Left or right end */
	End,

	Num
};
	
/**
 * Contains the basic primitives that are supposed to be drawn. This is designed to keep the drawing algorithm as straight forward as possible.
 * This way, theoretically, we could unit test the geometry generation.
 */
struct FTweenSliderDrawArgs
{
	/** The black bar */
	FGeometry BarArea;
	
	/** The points on the bar */
	TArray<FGeometry> Points;
	/** Equal length as Points. Used to get the right brush. */
	TArray<EPointType> PointTypes;
	/**
	 * Indicates which of the points should be rendered as passed (i.e. the slider has moved over them): true if passed, false if not passed.
	 * Equal length as Points when dragging. 0 length if not dragging. 
	 */
	TBitArray<> PassedPoints;
	
	/** The slider button. */
	FGeometry SliderArea;
	/** The area of icon in the button */
	FGeometry IconArea;

	/** This widget is shown while dragging: it draws from the center to the slider. It helps the user see how much they dragged so far. */
	FGeometry DragValueIndication;
	
	/** Whether to draw the button as pressed down. Can be true when bIsDragging is false (e.g. while detecting a drag). */
	bool bDrawButtonPressed = false;
	/** Whether the slider is being dragged. Affects whether to draw DragValueIndication and how to draw points. */
	bool bIsDragging = false;
	
	/** Indicates the hovered element */
	FTweenSliderHoverState HoverState;
};

/** The drawing relevant construction args of the widget. */
struct FTweenWidgetArgs
{
	/** The style that was used to construct the widget. */
	const FTweenSliderStyle* Style;
	
	/** The root opacity of the widget. */
	TAttribute<FSlateColor> ColorAndOpacity;

	/** The icon to place in the button */
	TAttribute<const FSlateBrush*> SliderIconAttr;
	/** The main color. It tints the slider button and the points. */
	TAttribute<FLinearColor> SliderColor;
	/** Affects how the normalized values are supposed to be interpreted */
	TAttribute<ETweenScaleMode> ScaleRenderModeAttr;
	
	/** If set, an indication where to position the slider. Range [-1,1]. If unset, defaults to 0. Ignored if the user is dragging the slider. */
	TAttribute<TOptional<float>> OverrideSliderPositionAttr;
};

/** Gets geometry of the background bar. */
void GetBarGeometry(const FGeometry& AllottedGeometry, const FTweenWidgetArgs& InWidgetArgs, FGeometry& OutBarArea);
/** This widget is shown while dragging: it draws from the center to the slider. */
void GetDragValueIndicationGeometry(
	const FGeometry& InAllottedGeometry, const FTweenWidgetArgs& InWidgetArgs, float InSliderPosition, FGeometry& OutDragValueIndication
	);
/** Computes the geometry for the slider button */
void GetSliderButtonGeometry(
	float InNormalizedPosition, const FGeometry& InAllottedGeometry, const FTweenWidgetArgs& InWidgetArgs, FGeometry& OutSliderArea, FGeometry& OutIconArea
	);
	
/** Gets the drawn geometry of the points on the bar. */
void GetDrawnPointGeometry(
	const FGeometry& InAllottedGeometry, const FTweenWidgetArgs& InWidgetArgs,
	const FTweenSliderHoverState& InHoverState, bool bIsMouseButtonDown,
	TArray<FGeometry>& OutPoints, TArray<EPointType>& OutPointTypes, TArray<float>& OutNormalizedPositions
	);
/** Gets the point geometry for doing hit tests with. */
void GetPointHitTestGeometry(
	const FGeometry& InAllottedGeometry, const FTweenWidgetArgs& InWidgetArgs,
	TArray<FGeometry>& OutPoints, TArray<float>& OutPointSliderValues
	);
/** Util overload for when the caller does not care about the point values. */
void GetPointHitTestGeometry(
	const FGeometry& InAllottedGeometry, const FTweenWidgetArgs& InWidgetArgs, TArray<FGeometry>& OutPoints 
);

/** Computes the points that have been passed by the slider while dragging. */
void GetPassedPointStates(const TArray<float>& InNormalizedPositions, float InSliderPosition, TBitArray<>& OutPassedPoints);
/** Computes the hover state. */
FTweenSliderHoverState GetHoverState(const FVector2D& InMouseScreenSpace, const FGeometry& InButtonArea, const TArray<FGeometry>& InPoints);
	
/**
 * Draws a tween slider.
 * @return The max LayerId that was drawn onto.
 */
int32 DrawTweenSlider(
	const FTweenSliderDrawArgs& InDrawArgs,
	const FTweenWidgetArgs& InWidgetArgs,
	FSlateWindowElementList& OutDrawElements,
	int32 LayerId,
	const FWidgetStyle& InWidgetStyle
	);
}
