// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/TweenSliderStyle.h"

#include "Brushes/SlateColorBrush.h"
#include "Brushes/SlateImageBrush.h"
#include "Brushes/SlateRoundedBoxBrush.h"
#include "Styling/SlateColor.h"
#include "Styling/StyleColors.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TweenSliderStyle)

const FName FTweenSliderStyle::TypeName( TEXT("FTweenSliderStyle") );

FTweenPointStyle::FTweenPointStyle() = default;

FTweenPointStyle::FTweenPointStyle(
	const FVector2D& InNormalSize, const FVector2D& InHoveredSize, const FVector2D& InPressedSize, const FVector2D& InHitSize
	)
	: Normal(static_cast<FSlateBrush>(FSlateImageBrush(NAME_None, InNormalSize, FStyleColors::ForegroundHover)))
	, Hovered(static_cast<FSlateBrush>(FSlateImageBrush(NAME_None, InHoveredSize, FStyleColors::ForegroundHover)))
	, Pressed(static_cast<FSlateBrush>(FSlateImageBrush(NAME_None, InPressedSize, FStyleColors::White)))
	, PassedPoint(static_cast<FSlateBrush>(FSlateImageBrush(NAME_None, InNormalSize, FStyleColors::White)))
	, HitTestSize(InHitSize)
{}

void FTweenPointStyle::GetResources(TArray<const FSlateBrush*>& OutBrushes) const
{
	OutBrushes.Add(&Normal);
	OutBrushes.Add(&Hovered);
	OutBrushes.Add(&Pressed);
	OutBrushes.Add(&PassedPoint);
}

FTweenSliderStyle::FTweenSliderStyle()
	: BarDimensions(200.0, 12.0)
	, BarBrush(static_cast<FSlateBrush>(FSlateColorBrush(FStyleColors::Black)))
	, NormalSliderButton(static_cast<FSlateBrush>(FSlateRoundedBoxBrush(FStyleColors::ForegroundHover, 5.0f, FStyleColors::Transparent, 0.f)))
	, HoveredSliderButton(static_cast<FSlateBrush>(FSlateRoundedBoxBrush(FStyleColors::ForegroundHover, 5.0f, FStyleColors::Transparent, 0.f)))
	, PressedSliderButton(static_cast<FSlateBrush>(FSlateRoundedBoxBrush(FStyleColors::Foreground, 5.0f, FStyleColors::Transparent, 0.f)))
	, NormalIconTint(FStyleColors::Black)
	, HoveredIconTint(FStyleColors::White)
	, PressedIconTint(FStyleColors::White)
	, SmallPoint({ 4.0, 4.0 }, { 6.0, 6.0 }, { 6.0, 6.0 }, { 10.0, 12.0 })
	, MediumPoint({ 4.0, 8.0 }, { 6.0, 10.0 }, { 6.0, 10.0 }, { 10.0, 12.0 })
	, EndPoint({ 4.0, 12.0 }, { 6.0, 14.0 }, { 6.0, 14.0 }, { 10.0, 12.0 })
	, PassedValueBackground(static_cast<FSlateBrush>(FSlateColorBrush(FLinearColor(1.f, 1.f, 1.f, 0.3f))))
	, IconPadding(4.f, 2.f)
{
}

void FTweenSliderStyle::GetResources(TArray<const FSlateBrush*>& OutBrushes) const
{
	OutBrushes.Add(&BarBrush);
	OutBrushes.Add(&NormalSliderButton);
	OutBrushes.Add(&HoveredSliderButton);
	OutBrushes.Add(&PressedSliderButton);
	OutBrushes.Add(&PassedValueBackground);
	SmallPoint.GetResources(OutBrushes);
	MediumPoint.GetResources(OutBrushes);
	EndPoint.GetResources(OutBrushes);
}

const FTweenSliderStyle& FTweenSliderStyle::GetDefault()
{
	static FTweenSliderStyle Default;
	return Default;
}
