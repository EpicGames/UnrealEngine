// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Templates/FunctionFwd.h"
#include "Templates/SharedPointerFwd.h"

class FString;
class FText;
class FUICommandInfo;
struct FLinearColor;
struct FSlateBrush;

namespace UE::TweeningUtilsEditor
{
class FTweenModel;

/**
 * This acts as the source of FTweenModels displayed in UI, e.g. by FTweenToolbarController.
 * The implementation is expected to be a static list of tween functions, i.e. each invocation of ForEachFunction is supposed to return the same functions.
 */
class ITweenModelContainer
{
public:

	/** Iterates through all the functions */
	virtual void ForEachModel(const TFunctionRef<void(FTweenModel&)> InConsumer) const = 0;

	/** @return The function at InIndex. */
	virtual FTweenModel* GetModel(int32 InIndex) const = 0;
	/** @return The number of functions contained. */
	virtual int32 NumModels() const = 0;
	/** @return The index of InTweenModel if found and INDEX_NONE if not. */
	TWEENINGUTILSEDITOR_API int32 IndexOf(const FTweenModel& InTweenModel) const;
	/** @return Whether InIndex is a valid argument for GetFunction. */
	bool IsValidIndex(int32 InIndex) const { return InIndex >= 0 && InIndex < NumModels(); }
	/** @return Whether InTweenModel is contained by this model. */
	bool Contains(const FTweenModel& InTweenModel) const { return IsValidIndex(IndexOf(InTweenModel)); }

	/** @return The command that is used to select the function */
	virtual TSharedPtr<FUICommandInfo> GetCommandForModel(const FTweenModel& InTweenModel) const = 0;

	/** @return The un-tinted icon to display the function in the UI with */
	virtual const FSlateBrush* GetIconForModel(const FTweenModel& InTweenModel) const = 0;
	/** @return The color that represents the function in the UI */
	virtual FLinearColor GetColorForModel(const FTweenModel& InTweenModel) const = 0;

	/** @return The label to display in the combo button */
	virtual FText GetLabelForModel(const FTweenModel& InTweenModel) const = 0;
	/** @return The description to displayed, e.g. in tool tips. */
	virtual FText GetToolTipForModel(const FTweenModel& InTweenModel) const = 0;

	/** @return Identifier that uniquely identifies this function. Used e.g. to encode the function type in a config file. */
	virtual FString GetModelIdentifier(const FTweenModel& InTweenModel) const = 0;
	/** @return The function identified by the InIdentifier (as returned by a call to GetFunctionIdentifier). */
	TWEENINGUTILSEDITOR_API FTweenModel* FindModelByIdentifier(const FString& InIdentifier);

	virtual ~ITweenModelContainer() = default;
};
}
