// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Templates/SharedPointerFwd.h"

#include <type_traits>

class FString;
class FText;
class FUICommandInfo;
struct FLinearColor;
struct FSlateBrush;
template<typename T> struct TOptional;

/** The following just contains some utils that is common in UI code. Tools can opt for their own description but using this helps with consistency. */
namespace UE::TweeningUtilsEditor
{
/** Util enum which is useful for UI code to abstract functions.*/
enum class EBlendFunction : uint8
{
	// Simple blends, i.e. signature: double(const FKeyBlendInfo&).
	BlendNeighbor,
	PushPull,
	BlendEase,
	ControlsToTween,
	BlendRelative,
	SmoothRough,

	// Complex blends
	TimeOffset,

	// Steps for adding a new function:
	// 1. Add the function to KeyBlendingFunctions.h (consider splitting implementation to KeyBlendingFunctions_[YourFunc].cpp).
	// 2. Extend this enum
	// 3. Extend FTweeningUtilsCommands with another command
	// 4. Extend FTweeningUtilsStyle with color, command style, and icon.
	// 5. Update GetFunctionData in KeyBlendingAbstraction.cpp
	// Done. Every system using the below functions will be up to date.
	//
	// Once you extend this  enum, you cannot miss the required steps as they're all guarded by static_asserts. If it compiles, you did everything.
	// Make sure you add static_assert(static_cast<int32>(EBlendFunction::Num) == x, "Instruction");, if you add additional steps in the future!

	// Add new entries above.
	Num
};

/** @return Converts InFunction to a string */
TWEENINGUTILSEDITOR_API FString BlendFunctionToString(EBlendFunction InFunction);
/** @return The EBlendFunction encoded by InString. */
TWEENINGUTILSEDITOR_API TOptional<EBlendFunction> LexBlendFunction(const FString& InString);

/** @return The command that is used to select the function in most UI. */
TWEENINGUTILSEDITOR_API TSharedPtr<FUICommandInfo> GetCommandForBlendFunction(EBlendFunction InFunction);
/** @return The un-tinted icon to display the function in the UI with */
TWEENINGUTILSEDITOR_API	const FSlateBrush* GetUntintedIconForTweenFunction(EBlendFunction InFunction);
/** @return The color that represents the function in the UI */
TWEENINGUTILSEDITOR_API FLinearColor GetTintColorForTweenFunction(EBlendFunction InFunction);
	
/** @return The full label to display in the function with, e.g. "Push / Pull" */
TWEENINGUTILSEDITOR_API	FText GetLabelForBlendFunction(EBlendFunction InFunction);
/** @return The short label to display in the function with, e.g. "PP" (for Push / Pull) */
TWEENINGUTILSEDITOR_API	FText GetAbbreviationForBlendFunction(EBlendFunction InFunction);
/** @return The description of the function */
TWEENINGUTILSEDITOR_API	FText GetDescriptionForBlendFunction(EBlendFunction InFunction);

/** Iterates through all blend function types. */
template<typename TCallback> requires std::is_invocable_v<TCallback, EBlendFunction>
void ForEachBlendFunction(TCallback&& InCallback);
	
enum class EBreakBehavior : uint8 { Continue, Break };
/** Iterates through all blend function types with the ability to break. */
template<typename TCallback> requires std::is_invocable_r_v<EBreakBehavior, TCallback, EBlendFunction>
void ForEachBlendFunctionBreakable(TCallback&& InCallback);
}

namespace UE::TweeningUtilsEditor
{
template<typename TCallback> requires std::is_invocable_v<TCallback, EBlendFunction>
void ForEachBlendFunction(TCallback&& InCallback)
{
	ForEachBlendFunctionBreakable([&InCallback](EBlendFunction BlendFunction)
	{
		InCallback(BlendFunction); return EBreakBehavior::Continue;
	});
}
template<typename TCallback> requires std::is_invocable_r_v<EBreakBehavior, TCallback, EBlendFunction>
void ForEachBlendFunctionBreakable(TCallback&& InCallback)
{
	for (int32 Index = 0; Index < static_cast<int32>(EBlendFunction::Num); ++Index)
	{
		if (InCallback(static_cast<EBlendFunction>(Index)) == EBreakBehavior::Break)
		{
			break;
		}
	}
}
}