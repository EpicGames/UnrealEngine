// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ITweenModelContainer.h"
#include "Math/Models/TweenModel.h"
#include "TweenModelDisplayInfo.h"
#include "Templates/UnrealTemplate.h"

#define UE_API TWEENINGUTILSEDITOR_API

namespace UE::TweeningUtilsEditor
{
/** Backed by an array of tween models and their UI info. */
class FTweenModelArray : public ITweenModelContainer, public FNoncopyable
{
public:
	
	explicit FTweenModelArray(TArray<FTweenModelUIEntry> InTweenModels) : TweenModels(MoveTemp(InTweenModels)) {}

	//~ Begin FTweenModelContainer Interface
	UE_API virtual void ForEachModel(const TFunctionRef<void(FTweenModel&)> InConsumer) const override;
	UE_API virtual FTweenModel* GetModel(int32 InIndex) const override;
	virtual int32 NumModels() const override { return TweenModels.Num(); }
	UE_API virtual TSharedPtr<FUICommandInfo> GetCommandForModel(const FTweenModel& InTweenModel) const override;
	UE_API virtual const FSlateBrush* GetIconForModel(const FTweenModel& InTweenModel) const override;
	UE_API virtual FLinearColor GetColorForModel(const FTweenModel& InTweenModel) const override;
	UE_API virtual FText GetLabelForModel(const FTweenModel& InTweenModel) const override;
	UE_API virtual FText GetToolTipForModel(const FTweenModel& InTweenModel) const override;
	UE_API virtual FString GetModelIdentifier(const FTweenModel& InTweenModel) const override;
	//~ Begin FTweenModelContainer Interface

private:

	/** Info about the contained tween models. */
	const TArray<FTweenModelUIEntry> TweenModels;
};
}

#undef UE_API
