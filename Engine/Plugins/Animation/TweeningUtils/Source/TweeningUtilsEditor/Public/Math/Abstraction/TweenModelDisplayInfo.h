// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Internationalization/Text.h"
#include "KeyBlendingAbstraction.h"
#include "Math/Color.h"
#include "Math/Models/TweenModel.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"

class FUICommandInfo;

namespace UE::TweeningUtilsEditor
{
class FTweenModel;

/** Information about how a FTweenModel can be displayed in the UI. */
struct FTweenModelDisplayInfo
{
	TSharedPtr<FUICommandInfo> Command;
	const FSlateBrush* Brush;
	FLinearColor Color;
	FText Label;
	FText ToolTip;
	FString Identifier;

	explicit FTweenModelDisplayInfo(
		TSharedPtr<FUICommandInfo> Command, const FSlateBrush* Brush, const FLinearColor& Color,
		const FText& Label, const FText& ToolTip, const FString& Identifier
		)
		: Command(MoveTemp(Command))
		, Brush(Brush)
		, Color(Color)
		, Label(Label)
		, ToolTip(ToolTip)
		, Identifier(Identifier)
	{}

	explicit FTweenModelDisplayInfo(EBlendFunction InBlendFunction)
		: FTweenModelDisplayInfo(
			GetCommandForBlendFunction(InBlendFunction),
			GetUntintedIconForTweenFunction(InBlendFunction),
			GetTintColorForTweenFunction(InBlendFunction),
			GetLabelForBlendFunction(InBlendFunction),
			GetDescriptionForBlendFunction(InBlendFunction),
			BlendFunctionToString(InBlendFunction)
			)
	{}
};

/** An association between a tween model and its UI display info. Useful for backing some UI, e.g. like combo button selection list. */
struct FTweenModelUIEntry : FNoncopyable
{
	TUniquePtr<FTweenModel> TweenModel;
	FTweenModelDisplayInfo DisplayInfo;

	explicit FTweenModelUIEntry(TUniquePtr<FTweenModel> TweenModel, FTweenModelDisplayInfo DisplayInfo)
		: TweenModel(MoveTemp(TweenModel))
		, DisplayInfo(MoveTemp(DisplayInfo))
	{}
};
}
