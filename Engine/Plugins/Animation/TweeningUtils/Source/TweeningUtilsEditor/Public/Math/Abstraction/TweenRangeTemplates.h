// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/IntegerSequence.h"
#include "KeyBlendingAbstraction.h"
#include "Math/ContiguousKeyMapping.h"
#include "Math/KeyBlendingFunctions.h"

namespace UE::TweeningUtilsEditor
{
/**
 * Interface for applying "simple" blend function to range of keys.
 * Template specializations extract the required parameters and passes them to the underlying blend function.
 *
 * This pattern is only possible with "simple" blend functions, i.e. those that only have the generic parameters described in KeyBlendingFunctions.h.
 * For these simple functions, all you need to do is specialize TweenRange and your function should then show up automatically, provided tools are
 * using ForEachCurveTweenable to go discover tween simple functions.
 * 
 * Some blend functions may require more arguments. For example, time offsetting / shifting would require caching underlying FCurveModel to be able
 * to evaluate arbitrary function values between the keys, etc.
 */
template<EBlendFunction>
double TweenRange(
	double InBlendValue,
	const FBlendRangesData& AllBlendedKeys, const FContiguousKeys& CurrentBlendRange,
	int32 InCurrentKeyIndex
); // Intentionally not implemented. This function is supposed to be specialized with the EBlendFunction types that support it.

/**
 * @return Whether this EBlendFunction supports simple tweening by using TweenRange.
 * Some blend functions require more arguments, like a cached curve, etc.
 */
constexpr bool SupportsTweenRange(EBlendFunction BlendFunction);
/** @return Number of functions for which SupportsTweenRange returns true.  */
constexpr int32 NumBlendFunctionsSupportingTweenRange();
	
/**
 * Invokes InCallback for each EBlendFunction that TCurveTweener can be instantiated with.
 * InCallback must be a templated lambda, e.g. []<EBlendFunction Function>(){}.
 */
template <typename TCallback>
constexpr void ForEachCurveTweenable(TCallback&& InCallback);
}

namespace UE::TweeningUtilsEditor
{
constexpr bool SupportsTweenRange(EBlendFunction BlendFunction)
{
	// This function could use SFINAE but this way is easier to generate static_assert and remind developers to implement TweenRange with.
	static_assert(
		static_cast<int32>(EBlendFunction::Num) == 7,
		"Does the blend function you added support TweenRange? If so, specialize TweenRange and update this function."
		); // TweenRange is supported if all inputs to your tween function can be determined by one of the FContiguousKeyRanges::GetX functions.

	switch (BlendFunction)
	{
	case EBlendFunction::BlendNeighbor: [[fallthrough]];
	case EBlendFunction::PushPull: [[fallthrough]];
	case EBlendFunction::BlendEase: [[fallthrough]];
	case EBlendFunction::ControlsToTween: [[fallthrough]];
	case EBlendFunction::BlendRelative: [[fallthrough]];
	case EBlendFunction::SmoothRough:
		return true;
	default: return false;
	}
}

constexpr int32 NumBlendFunctionsSupportingTweenRange()
{
	int32 Num = 0;
	for (int32 FuncIdx = 0; FuncIdx < static_cast<int32>(EBlendFunction::Num); ++FuncIdx)
	{
		if (SupportsTweenRange(static_cast<EBlendFunction>(FuncIdx)))
		{
			++Num;
		}
	}
	return Num;
}

template<>
inline double TweenRange<EBlendFunction::ControlsToTween>(
	double InBlendValue,
	const FBlendRangesData& AllBlendedKeys, const FContiguousKeys& CurrentBlendRange,
	int32 InCurrentKeyIndex // Fyi, this particular blend function does not care for the current key, i.e. InCurrentKeyIndex
)
{
	return TweeningUtils::Blend_ControlsToTween(
		InBlendValue, AllBlendedKeys.GetBeforeBlendRange(CurrentBlendRange), AllBlendedKeys.GetAfterBlendRange(CurrentBlendRange)
		);
}

template<>
inline double TweenRange<EBlendFunction::PushPull>(
	double InBlendValue,
	const FBlendRangesData& AllBlendedKeys, const FContiguousKeys& CurrentBlendRange,
	int32 InCurrentKeyIndex
)
{
	return TweeningUtils::Blend_PushPull(
		InBlendValue,
		AllBlendedKeys.GetBeforeBlendRange(CurrentBlendRange),
		AllBlendedKeys.GetCurrent(CurrentBlendRange, InCurrentKeyIndex),
		AllBlendedKeys.GetAfterBlendRange(CurrentBlendRange)
		);
}
	
template<>
inline double TweenRange<EBlendFunction::BlendNeighbor>(
	double InBlendValue,
	const FBlendRangesData& AllBlendedKeys, const FContiguousKeys& CurrentBlendRange,
	int32 InCurrentKeyIndex
)
{
	return TweeningUtils::Blend_Neighbor(
		InBlendValue,
		AllBlendedKeys.GetBeforeBlendRange(CurrentBlendRange),
		AllBlendedKeys.GetCurrent(CurrentBlendRange, InCurrentKeyIndex),
		AllBlendedKeys.GetAfterBlendRange(CurrentBlendRange)
		);
}

template<>
inline double TweenRange<EBlendFunction::BlendRelative>(
	double InBlendValue,
	const FBlendRangesData& AllBlendedKeys, const FContiguousKeys& CurrentBlendRange,
	int32 InCurrentKeyIndex
)
{
	return TweeningUtils::Blend_Relative(
		InBlendValue,
		AllBlendedKeys.GetBeforeBlendRange(CurrentBlendRange),
		AllBlendedKeys.GetFirstInBlendRange(CurrentBlendRange),
		AllBlendedKeys.GetCurrent(CurrentBlendRange, InCurrentKeyIndex),
		AllBlendedKeys.GetLastInBlendRange(CurrentBlendRange),
		AllBlendedKeys.GetAfterBlendRange(CurrentBlendRange)
		);
}
	
template<>
inline double TweenRange<EBlendFunction::BlendEase>(
	double InBlendValue,
	const FBlendRangesData& AllBlendedKeys, const FContiguousKeys& CurrentBlendRange,
	int32 InCurrentKeyIndex
)
{
	return TweeningUtils::Blend_Ease(
		InBlendValue,
		AllBlendedKeys.GetBeforeBlendRange(CurrentBlendRange),
		AllBlendedKeys.GetCurrent(CurrentBlendRange, InCurrentKeyIndex),
		AllBlendedKeys.GetAfterBlendRange(CurrentBlendRange)
		);
}

template<>
inline double TweenRange<EBlendFunction::SmoothRough>(
	double InBlendValue,
	const FBlendRangesData& AllBlendedKeys, const FContiguousKeys& CurrentBlendRange,
	int32 InCurrentKeyIndex
)
{
	return TweeningUtils::Blend_SmoothRough(
		InBlendValue,
		AllBlendedKeys.GetBeforeCurrent(CurrentBlendRange, InCurrentKeyIndex),
		AllBlendedKeys.GetCurrent(CurrentBlendRange, InCurrentKeyIndex),
		AllBlendedKeys.GetAfterCurrent(CurrentBlendRange, InCurrentKeyIndex)
		);
}

template <typename TCallback>
constexpr void ForEachCurveTweenable(TCallback&& InCallback)
{
	static_assert(NumBlendFunctionsSupportingTweenRange() == 6, "Extend this function with the enum entry you added.");
	InCallback.template operator()<EBlendFunction::BlendNeighbor>();
	InCallback.template operator()<EBlendFunction::PushPull>();
	InCallback.template operator()<EBlendFunction::BlendEase>();
	InCallback.template operator()<EBlendFunction::ControlsToTween>();
	InCallback.template operator()<EBlendFunction::BlendRelative>();
	InCallback.template operator()<EBlendFunction::SmoothRough>();
}
}