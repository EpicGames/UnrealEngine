// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CurveDataAbstraction.h"
#include "CurveEditorTypes.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "HAL/Platform.h"
#include "Misc/CoreMiscDefines.h"

#define UE_API TWEENINGUTILSEDITOR_API

class FCurveEditor;

namespace UE::TweeningUtilsEditor
{
/** Unbroken chain of keys. */
struct FContiguousKeys
{
	/** Indices of keys to blend */
	const TArray<int32> Indices;
	/** The index of the key before Indices[0]. INDEX_NONE if there is no previous key. */
	const int32 PreviousIndex = INDEX_NONE;
	/** The index of the key after Indidces' last element. INDEX_NONE if there is no next key. */
	const int32 NextIndex = INDEX_NONE;
	
	explicit FContiguousKeys(const TArray<FVector2d>& InAllKeyPositions, const TArray<int32>& InContiguousKeyIndices)
		: Indices(InContiguousKeyIndices)
		, PreviousIndex(InAllKeyPositions.IsValidIndex(Indices[0] - 1) ? Indices[0] - 1 : INDEX_NONE)
		, NextIndex(InAllKeyPositions.IsValidIndex(Indices[Indices.Num() -1] + 1) ? Indices[Indices.Num() -1] + 1 : INDEX_NONE)
		{}
};

/**
 * Stores all key positions as they were before the blend, and all the chains of keys that are supposed to be blended.
 * You can use this type independently from FContiguousKeyMapping. For example, you can use this to call TweeningUtilsEditor::TweenRange directly.
 */
struct FBlendRangesData
{
	const TArray<FVector2d> AllKeyPositions;

	/**
	 * The user selected keys split up into consecutive ranges.
	 * 
	 * Let 'x' denote keys selected by the user and '.' keys not selected.
	 * Suppose the user selection was '.xx...xxx.', i.e. the user selected keys and then shift+selected additional keys somewhere else on the curve.
	 * Then this array would contain indices of the two ranges 'xx' and 'xxx' is not a blendable range.
	 */
	TArray<FContiguousKeys> KeysArray;

	explicit FBlendRangesData(const TArray<FVector2d>& InAllKeyPositions) : AllKeyPositions(InAllKeyPositions) {}

	/** @return The key before the blend range defined by InKeys */
	const FVector2d& GetBeforeBlendRange(const FContiguousKeys& InKeys) const;
	/** @retuirn The first key that is blended in the specified range. */
	const FVector2D& GetFirstInBlendRange(const FContiguousKeys& InKeys) const;

	/**
	 * @param InIndex Index to InKeys.Indices.
	 * @return Gets the key before the currently blended key. If the current key is the first in the blend range, this returns GetBeforeBlendRange.
	 */
	const FVector2D& GetBeforeCurrent(const FContiguousKeys& InKeys, int32 InIndex) const;
	/**
	 * @param InIndex Index to InKeys.Indices.
	 * @return AllKeyPositions[Indices[Index]] */
	const FVector2D& GetCurrent(const FContiguousKeys& InKeys, int32 InIndex) const;
	/**
	 * @param InIndex Index to InKeys.Indices.
	 * @return Gets the key after the currently blended key. If the current key is the last in the blend range, this returns GetAfterBlendRange.
	 */
	const FVector2D& GetAfterCurrent(const FContiguousKeys& InKeys, int32 InIndex) const;
	
	/** @retuirn The last key that is blended in the specified range. */
	const FVector2D& GetLastInBlendRange(const FContiguousKeys& InKeys) const;
	/** @return The key after the blend range defined by InKeys */
	const FVector2d& GetAfterBlendRange(const FContiguousKeys& InKeys) const;

	UE_API void AddBlendRange(const TArray<int32>& InContiguousKeyIndices);
};
	
/** Information for blending a map of curves and the keys in it to blend. Saves the old key values for the duration of the blend.*/
struct FContiguousKeyMapping
{
	/** Holds the corresponding key handles for all the keys. */
	struct FContiguousKeysArray : FBlendRangesData
	{
		const TArray<FKeyHandle> AllKeyHandles;
		
		explicit FContiguousKeysArray(const TArray<FKeyHandle>& InAllKeyHandles, const TArray<FVector2d>& InAllKeyPositions)
			: FBlendRangesData(InAllKeyPositions) , AllKeyHandles(InAllKeyHandles) {}
		
		UE_API void AddBlendRange(const TArray<int32>& ContiguousKeyIndices);
	};

	FContiguousKeyMapping() = default;
	UE_API explicit FContiguousKeyMapping(const FCurveEditor& InCurveEditor);

	/** Appends keys from the given curve.*/
	UE_API void Append(const FCurveEditor& InCurveEditor, const FCurveModelID& InCurveId, TConstArrayView<FKeyHandle> InKeysToBlend);

	TMap<FCurveModelID, FContiguousKeysArray> KeyMap;
};
}

namespace UE::TweeningUtilsEditor
{
inline const FVector2d& FBlendRangesData::GetBeforeBlendRange(const FContiguousKeys& InKeys) const
{
	const int32 BeforeIndex = InKeys.PreviousIndex != INDEX_NONE ? InKeys.PreviousIndex : InKeys.Indices[0];
	return AllKeyPositions[BeforeIndex];
}
	
inline const FVector2D& FBlendRangesData::GetFirstInBlendRange(const FContiguousKeys& InKeys) const
{
	return AllKeyPositions[InKeys.Indices[0]];
}
	
inline const FVector2D& FBlendRangesData::GetBeforeCurrent(const FContiguousKeys& InKeys, int32 InIndex) const
{
	return InIndex > 0 ? AllKeyPositions[InKeys.Indices[InIndex - 1]] : GetBeforeBlendRange(InKeys);
}

inline const FVector2D& FBlendRangesData::GetCurrent(const FContiguousKeys& InKeys, int32 InIndex) const
{
	return AllKeyPositions[InKeys.Indices[InIndex]];
}

inline const FVector2D& FBlendRangesData::GetAfterCurrent(const FContiguousKeys& InKeys, int32 InIndex) const
{
	return InIndex < InKeys.Indices.Num() - 1 ? AllKeyPositions[InKeys.Indices[InIndex + 1]] : GetAfterBlendRange(InKeys);
}
	
inline const FVector2D& FBlendRangesData::GetLastInBlendRange(const FContiguousKeys& InKeys) const
{
	return AllKeyPositions[InKeys.Indices[InKeys.Indices.Num() - 1]];
}

inline const FVector2d& FBlendRangesData::GetAfterBlendRange(const FContiguousKeys& InKeys) const
{
	const int32 AfterIndex = InKeys.NextIndex != INDEX_NONE ? InKeys.NextIndex : InKeys.Indices[InKeys.Indices.Num() - 1];
	return AllKeyPositions[AfterIndex];
}
}

#undef UE_API
