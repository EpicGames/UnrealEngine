// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ContiguousKeyMapping.h"
#include "CurveDataAbstraction.h"
#include "CurveEditor.h"
#include "CurveModel.h"

#include <type_traits>

namespace UE::TweeningUtilsEditor
{
/**
 * A callable that knows how to blend a key at a time.
 * Use AllBlendedKeys.AllKeyPositions[CurrentBlendRange.Indices[Index]] to get current key
 */
template<typename TCallback>
concept CBlendKeyCallable = std::is_invocable_r_v<double, TCallback,
	const FCurveModelID& /*BlendedCurve*/,
	const FContiguousKeyMapping::FContiguousKeysArray& /*AllBlendedKeys*/,
	const FContiguousKeys& /*CurrentBlendRange*/,
	int32 /*Index, i.e. the index into FContiguousKeys::Indices currently being blended.*/
	>;
/** Blends the specified keys by using a InBlendKeyCallback, which returns the new key value each key should have. */
template<CBlendKeyCallable TCallback>
bool BlendCurves_BySingleKey(const FCurveEditor& InCurveEditor, const FContiguousKeyMapping& InKeySelection, TCallback&& InBlendKeyCallback);
	
/**
 * A callable that knows how to blend an entire range of keys at a time.
 * 
 * Blend values are written out to OutKeyHandles & OutKeyPositions, which are preallocated to be
 * OutKeyHandles.Num() == OutKeyPositions.Num() == CurrentBlendRange.Indices.Num().
 */
template<typename TCallback>
concept CBlendRangeCallback = std::is_invocable_v<TCallback,
	const FCurveModelID& /*BlendedCurve*/,
	const FContiguousKeyMapping::FContiguousKeysArray& /*AllBlendedKeys*/,
	const FContiguousKeys& /*CurrentBlendRange*/,
	TArray<FKeyHandle>& /*OutKeyHandles*/,
	TArray<FKeyPosition>& /*OutKeyPositions*/
	>;
/**
 * Generic helper function that invokes InBlendRangeCallback for each unbroken sub-range of keys in each curve.
 * @return Whether any key value was changed.
 */
template<CBlendRangeCallback TCallback>
bool BlendCurve_ByKeyRange(const FCurveEditor& InCurveEditor, const FContiguousKeyMapping& InKeySelection, TCallback&& InBlendRangeCallback);
}

namespace UE::TweeningUtilsEditor
{
template <CBlendKeyCallable TCallback>
bool BlendCurves_BySingleKey(const FCurveEditor& InCurveEditor, const FContiguousKeyMapping& InKeySelection, TCallback&& InBlendKeyCallback)
{
	return BlendCurve_ByKeyRange(InCurveEditor, InKeySelection,
		[&InBlendKeyCallback](
			const FCurveModelID& CurveId,
			const FContiguousKeyMapping::FContiguousKeysArray& AllBlendedKeys, const FContiguousKeys& CurrentBlendRange,
			TArray<FKeyHandle>& OutKeyHandles, TArray<FKeyPosition>& OutKeyPositions
		)
		{
			const TConstArrayView<FKeyHandle>& AllKeyHandles = AllBlendedKeys.AllKeyHandles;
			const TConstArrayView<int32> Indices = CurrentBlendRange.Indices;

			for (int32 Index = 0; Index < Indices.Num(); ++Index)
			{
				const double NewValue = InBlendKeyCallback(CurveId, AllBlendedKeys, CurrentBlendRange, Index);
				const FVector2D& CurrentKey = AllBlendedKeys.GetCurrent(CurrentBlendRange, Index);
				OutKeyPositions[Index] = FKeyPosition(CurrentKey.X, NewValue);
				OutKeyHandles[Index] = AllKeyHandles[Indices[Index]];
			}
		});
}

template<CBlendRangeCallback TCallback>
bool BlendCurve_ByKeyRange(const FCurveEditor& InCurveEditor, const FContiguousKeyMapping& InKeySelection, TCallback&& InBlendRangeCallback)
{
	bool bDidBlend = false;
	
	TArray<FKeyHandle> KeyHandles;
	TArray<FKeyPosition> KeyPositions;
	for (const TPair<FCurveModelID, FContiguousKeyMapping::FContiguousKeysArray>& KeysArray : InKeySelection.KeyMap)
	{
		const FCurveModelID& ModelId = KeysArray.Key;
		FCurveModel* Curve = InCurveEditor.FindCurve(ModelId);
		if (!Curve)
		{
			continue;
		}
		
		const FContiguousKeyMapping::FContiguousKeysArray& CurveBlendedKeys = KeysArray.Value;
		for (const FContiguousKeys& BlendRange : CurveBlendedKeys.KeysArray)
		{
			const int32 NumIndices = BlendRange.Indices.Num();
			KeyHandles.SetNum(NumIndices);
			KeyPositions.SetNum(NumIndices);

			InBlendRangeCallback(ModelId, CurveBlendedKeys, BlendRange, KeyHandles, KeyPositions);
				
			Curve->SetKeyPositions(KeyHandles, KeyPositions);
			bDidBlend = true;
		}
	}

	return bDidBlend;
}
}
