// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IBufferedCurveModel.h"
#include "TweenModel.h"
#include "Math/ContiguousKeyMapping.h"
#include "Misc/Attribute.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

#define UE_API TWEENINGUTILSEDITOR_API

class FCurveEditor;

namespace UE::TweeningUtilsEditor
{
/** Recomputes all keys output values such that the curve is effectively shifted to left and right without modifying the keys' input values. */
class FCurveTimeOffsetTweenModel : public FTweenModel, public FNoncopyable
{
public:

	UE_API explicit FCurveTimeOffsetTweenModel(TAttribute<TWeakPtr<FCurveEditor>> InWeakCurveEditor);
	
	//~ Begin FTweenModel Interface
	UE_API virtual void StartBlendOperation() override;
	UE_API virtual void StopBlendOperation() override;
	UE_API virtual void BlendValues(float InNormalizedValue) override;
	//~ Begin FTweenModel Interface

protected:
	
	/** The curve editor on which to tween the curves. */
	const TAttribute<TWeakPtr<FCurveEditor>> WeakCurveEditor;

	/** Created in StartBlendOperation and used for the entirety of the blend operation. */
	FContiguousKeyMapping ContiguousKeySelection;

	/** The state of the curves before they were blended. Created in StartBlendOperation. Used to evaluate original curve value during time shift. */
	TMap<FCurveModelID, TUniquePtr<IBufferedCurveModel>> OriginalBlendedCurves;
};
}

#undef UE_API
