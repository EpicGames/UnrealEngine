// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CurveEditor.h"
#include "Math/Abstraction/TweenRangeTemplates.h"
#include "Math/ContiguousKeyMapping.h"
#include "Math/CurveBlending.h"
#include "TweenModel.h"
#include "Templates/SharedPointer.h"

class FCurveEditor;

namespace UE::TweeningUtilsEditor
{
/** Tweens the curves of the curve editor's selection. */
template<EBlendFunction Func>
class TCurveTweenModel : public FTweenModel
{
	static_assert(SupportsTweenRange(Func), "This EBlendFunction does not support TweenRange<EBlendFunction>");
public:
	
	explicit TCurveTweenModel(TWeakPtr<FCurveEditor> InWeakCurveEditor);

	/** @return Whether it makes sense to call StartBlendOperation, etc. */
	bool HasAnythingToBlend() const;
	
	//~ Begin FTweenModel Interface
	virtual void StartBlendOperation() override;
	virtual void StopBlendOperation() override;
	virtual void BlendValues(float InNormalizedValue) override;
	//~ Begin FTweenModel Interface

protected:
	
	/** The curve editor on which to tween the curves. */
	const TWeakPtr<FCurveEditor> WeakCurveEditor;

	/** Created in StartBlendOperation and used for the entirety of the blend operation. */
	FContiguousKeyMapping ContiguousKeySelection;
};
}

namespace UE::TweeningUtilsEditor
{
template <EBlendFunction BlendFunction>
TCurveTweenModel<BlendFunction>::TCurveTweenModel(TWeakPtr<FCurveEditor> InWeakCurveEditor)
	: WeakCurveEditor(MoveTemp(InWeakCurveEditor))
{}

template <EBlendFunction BlendFunction>
bool TCurveTweenModel<BlendFunction>::HasAnythingToBlend() const
{
	const TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	return CurveEditor && !CurveEditor->GetSelection().IsEmpty();
}

template <EBlendFunction BlendFunction>
void TCurveTweenModel<BlendFunction>::StartBlendOperation()
{
	const TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	ContiguousKeySelection = CurveEditor ? FContiguousKeyMapping(*CurveEditor) : FContiguousKeyMapping{};
}

template <EBlendFunction BlendFunction>
void TCurveTweenModel<BlendFunction>::StopBlendOperation()
{
	ContiguousKeySelection.KeyMap.Empty(
		ContiguousKeySelection.KeyMap.Num()
		);
}

template <EBlendFunction BlendFunction>
void TCurveTweenModel<BlendFunction>::BlendValues(float InNormalizedValue)
{
	const TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (!CurveEditor)
	{
		return;
	}

	const float ScaledBlendValue = ScaleBlendValue(InNormalizedValue); 
	TweeningUtilsEditor::BlendCurves_BySingleKey(*CurveEditor, ContiguousKeySelection,
		[this, ScaledBlendValue](
			const FCurveModelID&,
			const FContiguousKeyMapping::FContiguousKeysArray& AllBlendedKeys, const FContiguousKeys& CurrentBlendRange,
			int32 InCurrentKeyIndex
		)
	{
		return TweenRange<BlendFunction>(ScaledBlendValue, AllBlendedKeys, CurrentBlendRange, InCurrentKeyIndex);
	});
}
}