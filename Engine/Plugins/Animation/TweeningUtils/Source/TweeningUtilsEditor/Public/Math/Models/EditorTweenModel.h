// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TangentFlatteningTweenProxy.h"
#include "TweenModel.h"
#include "TransactionalTweenModelProxy.h"

#include <type_traits>

namespace UE::TweeningUtilsEditor
{
/**
 * Extends TUnderlyingModel with commonly expected shared functionality:
 *	- Undo / redo
 *	- As the tween function squishes the keys vertically, user-specified tangents are flattened.
 *	- In the future this can be extended to remove vertically stacked keys (same keys per frame). For now, no implemented tween function stack keys.
 *
 * Construct like this:
 *  using FModel = TEditorTweenModel<FFooTweenModel> // FFooTweenModel should inherit from FTweenModel.
 *	const TSharedRef<FModel> TweenModel = MakeShared<FModel>(
 *		WeakCurveEditor,							// TTangentFlatteningTweenProxy arg
 *		Bar											// Any arguments that FFooTweenModel requires for construction
 *	);
 */
template<typename TUnderlyingModel> requires std::is_base_of_v<FTweenModel, TUnderlyingModel>
using TEditorTweenModel = TTransactionalTweenModelProxy<TTangentFlatteningTweenProxy<TUnderlyingModel>>;
}
