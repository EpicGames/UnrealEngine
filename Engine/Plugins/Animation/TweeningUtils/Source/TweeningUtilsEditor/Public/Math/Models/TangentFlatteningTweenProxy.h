// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CurveEditor.h"
#include "Math/Models/TweenModel.h"
#include "Misc/Attribute.h"
#include "Misc/Mirror/TangentSelectionFlattener.h"
#include <type_traits>

#include "Misc/Mirror/TangentSingleKeyFlattener.h"

namespace UE::TweeningUtilsEditor
{
/**
 * This class squishes the curves based on how much the tween function squishes the keys vertically.
 * The squishing is achieved by interpolating the tangents to 0.
 */
template<typename TBase> requires std::is_base_of_v<FTweenModel, TBase>
class TTangentFlatteningTweenProxy : public TBase
{
public:

	template<typename... TArg>
	explicit TTangentFlatteningTweenProxy(TAttribute<TWeakPtr<FCurveEditor>> WeakCurveEditorAttr, TArg&&... Arg)
		: TBase(Forward<TArg>(Arg)...)
		, WeakCurveEditor(MoveTemp(WeakCurveEditorAttr))
	{
		check(WeakCurveEditorAttr.IsBound() || WeakCurveEditorAttr.IsSet());
	}

	//~ Begin FTweenModel Interface
	virtual void StartBlendOperation() override
	{
		TBase::StartBlendOperation();
		
		if (const TSharedPtr<FCurveEditor> CurveEditorPin = WeakCurveEditor.Get().Pin())
		{
			TangentTweener.ResetFromSelection(*CurveEditorPin);
			SingleKeyTangentTweener.ResetFromSelection(*CurveEditorPin);
		}
	}
	virtual void BlendValues(float InNormalizedValue) override
	{
		using namespace CurveEditor;
		
		TBase::BlendValues(InNormalizedValue);
		if (const TSharedPtr<FCurveEditor> CurveEditorPin = WeakCurveEditor.Get().Pin())
		{
			const double FinalBlendValue = TBase::ScaleBlendValue(InNormalizedValue);
			
			// Conceptually, we've got two horizontal, parallel edges: the top one, which moves InNormalizedValue goes from 0 to +/- 1, and the bottom edge at 0.
			// The tangents move depending on the edge relative distance. During overshoot mode, if the value exceeds +/- 1, the top edge has crossed the 0 edge.
			const bool bIsOvershooting = FMath::Abs(FinalBlendValue) > 1.f;
			const bool bTopEdgeHasCrossedBottomEdge = bIsOvershooting;
			TangentTweener.ComputeMirroringParallel(*CurveEditorPin, bTopEdgeHasCrossedBottomEdge);
			
			const ESide NeighborToBlendTowards = FinalBlendValue < 0.f ? ESide::Left : ESide::Right;
			SingleKeyTangentTweener.ComputeMirroringParallel(*CurveEditorPin, NeighborToBlendTowards);
		}
	}
	//~ Begin FTweenModel Interface

private:

	/** Needed as arg for TangentTweener. */
	const TAttribute<TWeakPtr<FCurveEditor>> WeakCurveEditor;
	/** Implements the logic for flattening tangents of consecutive keys. */
	CurveEditor::FTangentSelectionFlattener TangentTweener;
	/** Implements logic for flattening tangents of single keys */
	CurveEditor::FTangentSingleKeyFlattener SingleKeyTangentTweener;
};
}

