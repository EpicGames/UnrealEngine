// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TweenModel.h"

#include "CurveEditor.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Misc/Attribute.h"
#include "Modification/Utils/ScopedCurveChange.h"
#include "Templates/UniquePtr.h"

#include <type_traits>


class FCurveEditor;

namespace UE::TweeningUtilsEditor
{
/** Starts a transaction in StartBlendOperation, stops it in StopBlendOperation.  */
template<typename TBase> requires std::is_base_of_v<FTweenModel, TBase>
class TTransactionalTweenModelProxy : public TBase
{
public:

	template<typename... TArg>
	explicit TTransactionalTweenModelProxy(TAttribute<TWeakPtr<FCurveEditor>> InCurveEditorAttr, TArg&&... Arg)
		: TBase(Forward<TArg>(Arg)...)
		, CurveEditorAttr(MoveTemp(InCurveEditorAttr))
	{}
	
	//~ Begin FTweenModel Interface
	virtual void StartBlendOperation() override
	{
		if (const TSharedPtr<FCurveEditor> CurveEditor = CurveEditorAttr.Get().Pin())
		{
			InProgressTransaction.Emplace(
				CurveEditor::FCurvesSnapshotBuilder(CurveEditor).TrackAllCurves(),
				NSLOCTEXT("FTransactionalTweenModelProxy", "Transaction", "Blend values")
				);
		}
		TBase::StartBlendOperation();
	}
	virtual void StopBlendOperation() override
	{
		TBase::StopBlendOperation();
		InProgressTransaction.Reset();
	}
	//~ Begin FTweenModel Interface

private:

	/** Used to detect key changes. */
	const TAttribute<TWeakPtr<FCurveEditor>> CurveEditorAttr;
	
	/** Active during a blend operation. */
	TOptional<CurveEditor::FScopedCurveChange> InProgressTransaction;
};
}

