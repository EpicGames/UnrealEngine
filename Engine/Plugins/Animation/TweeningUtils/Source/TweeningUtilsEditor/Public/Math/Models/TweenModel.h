// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/ETweenScaleMode.h"

namespace UE::TweeningUtilsEditor
{
/**
 * Implements the logic of tweening some values: it could be curves, it could be control rig models.
 * Acts as model in a Model-View-Controller architecture.
 */
class FTweenModel
{
public:

	/** Called when a blend operation is started. */
	virtual void StartBlendOperation() {}
	/** Called when a blend operation is stopped. */
	virtual void StopBlendOperation() {}
	/**
	 * Does the blending based on the InNormalizedValue.
	 * @param InNormalizedValue Value in [-1, 1]
	 */
	virtual void BlendValues(float InNormalizedValue) = 0;

	/**
	 * Blends to a single value once.
	 * @param InNormalizedValue Value in [-1, 1]
	 */
	void BlendOneOff(float InNormalizedValue)
	{
		StartBlendOperation();
		BlendValues(InNormalizedValue);
		StopBlendOperation();
	}

	virtual void SetScaleMode(ETweenScaleMode InMode) { ScaleMode = InMode; }
	ETweenScaleMode GetScaleMode() const { return ScaleMode; }
	
	/**
	 * @param InNormalizedValue Value from -1 to 1.
	 * @return Value scaled according to the ScaleMode setting.
	 */
	float ScaleBlendValue(float InNormalizedValue) const
	{
		return ScaleMode == ETweenScaleMode::Normalized
			? InNormalizedValue
			: 2.f * InNormalizedValue;
	}
	
	virtual ~FTweenModel() = default;
	
private:

	/**
	 * Affects how normalized values sent to BlendValueNormalizedAndRecomputeKeys are interpreted:
	 * - False: -1.0 to 1.0 maps to -100% to +100%
	 * - True: -1.0 to 1.0 maps to -200% to +200%
	 */
	ETweenScaleMode ScaleMode = ETweenScaleMode::Normalized;
};
}

