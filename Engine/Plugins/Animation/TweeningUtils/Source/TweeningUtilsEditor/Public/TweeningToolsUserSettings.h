// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "UObject/Object.h"
#include "UObject/NameTypes.h"
#include "TweeningToolsUserSettings.generated.h"

#define UE_API TWEENINGUTILSEDITOR_API

UCLASS(MinimalAPI, Config=EditorPerProjectUserSettings)
class UTweeningToolsUserSettings : public UObject
{
	GENERATED_BODY()
public:

	static UE_API UTweeningToolsUserSettings* Get();

	/** Sets the preferred tween function for the given feature. */
	void SetPreferredTweenFunction(FName InFeatureKey, FString InNewPreferredFunction)
	{
		PreferredTweenFunction.Add(InFeatureKey, MoveTemp(InNewPreferredFunction));
		SaveConfig();
	}
	/** Gets the preferred tween function for the given feature, if there is any. */
	const FString* GetPreferredTweenFunction(FName InFeatureKey) const { return PreferredTweenFunction.Find(InFeatureKey); }
	
private:
	
	/**
	 * Associates features with the preferred function in that feature.
	 * If you want multiple locations in the editor to share the same setting, use the same key.
	 * This tween function should be selected by default when a curve editor is created. It is the function that was used last time in the editor.
	 */
	UPROPERTY(Config)
	TMap<FName, FString> PreferredTweenFunction;
};

#undef UE_API
