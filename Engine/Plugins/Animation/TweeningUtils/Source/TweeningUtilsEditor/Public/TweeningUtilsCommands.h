// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"

namespace UE::TweeningUtilsEditor
{
class TWEENINGUTILSEDITOR_API FTweeningUtilsCommands : public TCommands<FTweeningUtilsCommands>
{
public:
	
	TSharedPtr<FUICommandInfo> SetControlsToTween;
	TSharedPtr<FUICommandInfo> SetTweenPushPull;
	TSharedPtr<FUICommandInfo> SetTweenBlendNeighbor;
	TSharedPtr<FUICommandInfo> SetTweenBlendRelative;
	TSharedPtr<FUICommandInfo> SetTweenBlendEase;
	TSharedPtr<FUICommandInfo> SetTweenSmoothRough;
	TSharedPtr<FUICommandInfo> SetTweenTimeOffset;
	TSharedPtr<FUICommandInfo> ToggleOvershootMode;

	/** Triggers detection of mouse movement to drive the slider indirectly. */
	TSharedPtr<FUICommandInfo> DragAnimSliderTool;
	/** Cycles to the next tween functions */
	TSharedPtr<FUICommandInfo> ChangeAnimSliderTool;
	
	/** Positions the slider 100% to the left as if you had dragged it there. */
	TSharedPtr<FUICommandInfo> SetSliderLeft;
	/** Positions the slider 100% to the right as if you had dragged it there. */
	TSharedPtr<FUICommandInfo> SetSliderRight;
	/** Positions the slider 50% to the left as if you had dragged it there. */
	TSharedPtr<FUICommandInfo> SetSliderHalfLeft;
	/** Positions the slider 50% to the right as if you had dragged it there. */
	TSharedPtr<FUICommandInfo> SetSliderHalfRight;
	/** Positions the slider 25% to the left as if you had dragged it there. */
	TSharedPtr<FUICommandInfo> SetSliderQuarterLeft;
	/** Positions the slider 25% to the right as if you had dragged it there. */
	TSharedPtr<FUICommandInfo> SetSliderQuarterRight;

	FTweeningUtilsCommands();

	//~ Begin TCommands Interface
	virtual void RegisterCommands() override;
	//~ End TCommands Interface
};
}

