// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

namespace UE::TweeningUtilsEditor
{
/** Style for the widgets and commands in this module. */
class FTweeningUtilsStyle final
: public FSlateStyleSet
{
public:
	
	TWEENINGUTILSEDITOR_API static FTweeningUtilsStyle& Get();
	
	FTweeningUtilsStyle();
	virtual ~FTweeningUtilsStyle() override;
};
}
