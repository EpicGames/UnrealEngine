// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

namespace UE::TweeningUtilsEditor
{
/** Affects how blend values, which are normalized to the range -1.0 to 1.0, are interpreted. */
enum class ETweenScaleMode : uint8
{
	/** -1.0 to 1.0 maps to -100% to +100% */
	Normalized,
	/** -1.0 to 1.0 maps to -200% to +200% */
	Overshoot
};
}