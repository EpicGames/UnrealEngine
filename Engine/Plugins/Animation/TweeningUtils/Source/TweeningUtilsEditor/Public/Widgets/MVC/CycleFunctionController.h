// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Misc/Attribute.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

#define UE_API TWEENINGUTILSEDITOR_API

class FUICommandInfo;
class FUICommandList;

namespace UE::TweeningUtilsEditor
{
class FTweenModel;
class ITweenModelContainer;

/** Cycles through functions on a tween model. */
class FCycleFunctionController : public FNoncopyable
{
public:

	DECLARE_DELEGATE_OneParam(FHandleTweenChange, FTweenModel&);
	
	/**
	 * @param InCurrentTweenModelAttr The tween model that is currently selected.
	 * @param InTweenModelContainer Determines the functions through which you can cycle
	 * @param InHandleTweenChange Invoked to change the current tween function
	 * @param InCommandList The command list to add / remove the command to / from.
	 * @param InCycleCommand The command that cycles the function
	 */
	UE_API explicit FCycleFunctionController(
		TAttribute<const FTweenModel*> InCurrentTweenModelAttr,
		const TSharedRef<ITweenModelContainer>& InTweenModelContainer,
		FHandleTweenChange InHandleTweenChange,
		const TSharedRef<FUICommandList>& InCommandList,
		TSharedPtr<FUICommandInfo> InCycleCommand
		);
	/** Version that defauls the cycle command to FTweenUtilsCommands::ChangeAnimSliderTool. */
	UE_API explicit FCycleFunctionController(
		TAttribute<const FTweenModel*> InCurrentTweenModelAttr,
		const TSharedRef<ITweenModelContainer>& InTweenModelContainer,
		FHandleTweenChange InHandleTweenChange,
		const TSharedRef<FUICommandList>& InCommandList
		); 
	UE_API virtual ~FCycleFunctionController();

	DECLARE_MULTICAST_DELEGATE(FOnFunctionCycled);
	/** Invoked when CycleCommand is invoked. */
	FOnFunctionCycled& OnFunctionCycleCommandInvoked() { return OnFunctionCycleCommandInvokedDelegate; }
	
private:
	
	/** The tween model that is currently selected */
	const TAttribute<const FTweenModel*> CurrentTweenModelAttr;
	/** Holds the functions that can be cycled through. */
	const TSharedRef<ITweenModelContainer> TweenModelContainer;
	/** Invoked to change the current tween function */
	const FHandleTweenChange HandleTweenChangeDelegate;
	
	/** Used to bind and unbind the CycleCommand command. */
	const TSharedRef<FUICommandList> CommandList;
	/** Invokes CycleToNextFunction. */
	const TSharedPtr<FUICommandInfo> CycleCommand;

	/** Invoked when CycleCommand is invoked. */
	FOnFunctionCycled OnFunctionCycleCommandInvokedDelegate;
	
	/** Sets the next tween function. */
	UE_API void CycleToNextFunction() const;
};
}

#undef UE_API
