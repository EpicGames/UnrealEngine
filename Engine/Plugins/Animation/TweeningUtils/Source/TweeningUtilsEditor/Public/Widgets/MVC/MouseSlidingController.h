// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Math/Vector2D.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

#define UE_API TWEENINGUTILSEDITOR_API

class FUICommandInfo;
class FUICommandList;
class ICursor;

namespace UE::TweeningUtilsEditor
{
class FMouseSlidingInputProcessor;
class FTweenModel;

/**
 * Generic controller that allows you to move the mouse while a command is pressed.
 * The sliding value is normalized to [-1,1] depending on whether the mouse was moved left or right.
 * 
 * The user must first press down the key chord for DragSliderCommand, and then the LMB. The user can press and release LMB multiple times while
 * DragSliderCommand's key chord is pressed down.
 * 
 * During sliding, the mouse is locked to the rect of the virtual slider: the area around where the user started dragging.
 * This is intended to give the user feedback when they've moved the mouse far enough to reach -1 or 1.
 * 
 * Example:
 * - Suppose MaxSlideWidthAttr returns 200, and that InDragSliderCommand is bound to U.
 * - The user presses U and moves the mouse
 *		- to right by 50 Slate units > OnSliderMovedDelegate is invoked with is 0.5.
 *		- to the right by 125 Slate units (exceeds the max sliding by 25 units) > OnSliderMovedDelegate is invoked with 1.0 (clamped).
 *		- to the left by 25 Slate units > OnSliderMovedDelegate is invoked with -0.25.
 */
class FMouseSlidingController : public FNoncopyable
{
	friend FMouseSlidingInputProcessor;
public:
	
	/**
	 * @param InMaxSlideWidthAttr The total size of the "invisible" slider. The mouse can be moved left and right InMaxSlideWidthAttr.Get / 2.0 before being clamped.
	 * @param InCommandList The command list to add / remove the command to / from.
	 * @param InDragSliderCommand The command that triggers detection of mouse movement.
	 */
	UE_API FMouseSlidingController(
		TAttribute<float> InMaxSlideWidthAttr,
		const TSharedRef<FUICommandList>& InCommandList,
		TSharedPtr<FUICommandInfo> InDragSliderCommand
		);
	UE_API virtual ~FMouseSlidingController();

	/** Starts listening for the mouse sliding command. */
	UE_API void BindCommand();
	/** Stops listening for the mouse sliding command. */
	UE_API void UnbindCommand() const;

	/** @return Whether the slider operation is currently in progress. */
	bool IsSliding() const { return SlidingState.IsSet() && SlidingState->InitialMouse.IsSet(); }
	
	DECLARE_MULTICAST_DELEGATE_OneParam(FMoveSliderDelegate, float /*InNormalizedRange*/);
	/** Invoked every tick the slider is moved. The event is throttled, so there's at most one invocation per tick. */
	FMoveSliderDelegate& OnUpdateSliding() { return OnUpdateSlidingDelegate; }
	/** Invoked when sliding starts. */
	FSimpleMulticastDelegate& OnStartSliding() { return OnStartSlidingDelegate; }
	/** Invoked when sliding stops. */
	FSimpleMulticastDelegate& OnStopSliding() { return OnStopSlidingDelegate; }

protected:

	const TSharedRef<FUICommandList>& GetCommandList() const { return CommandList; }
	
private:
	
	/** The total size of the "invisible" slider. The mouse can be moved left and right by MaxSlideWidthAttr.Get() / 2.0 before being clamped. */
	const TAttribute<float> MaxSlideWidthAttr;

	/** Used to bind and unbind the DragSliderCommand command. */
	const TSharedRef<FUICommandList> CommandList;
	/** The command must be pressed down to start sliding. You also need to press LMB to start sliding. */
	const TSharedPtr<FUICommandInfo> DragSliderCommand;

	struct FSlidingState
	{
		/** Listens for DragSliderCommand and LMB going up. Orchestrates calls to StartSliding, StopSliding, and UpdateSliding. */
		const TSharedRef<FMouseSlidingInputProcessor> InputProcessor;
		/** Set while sliding is active. Unset while inactive. */
		TOptional<FVector2D> InitialMouse;

		explicit FSlidingState(FMouseSlidingController& InOwner);
		~FSlidingState();

		/** @return Whether sliding is currently active. */
		bool IsSliding() const { return InitialMouse.IsSet(); }
	};
	/** Set while listening for mouse events, unset while not sliding. */
	TOptional<FSlidingState> SlidingState;
	
	/** Invoked every tick the slider is moved. The event is throttled, so there's at most one invocation per tick. */
	FMoveSliderDelegate OnUpdateSlidingDelegate;
	/** Invoked when sliding starts. */
	FSimpleMulticastDelegate OnStartSlidingDelegate;
	/** Invoked when sliding stops. */
	FSimpleMulticastDelegate OnStopSlidingDelegate;

	/** Once DragSliderCommand has triggered, start listening for mouse down events. */
	void StartListeningForMouseEvents();
	/** Once DragSliderCommand is released, stop listening for mouse down events.  */
	void StopListeningForMouseEvents(ICursor& InCursor);

	/** Starts sliding from mouse position and locks the mouse; emits OnUpdateSliding calls. Called once DragSliderCommand and LMB are both down. */
	void StartSliding(const FVector2D& InInitialScreenLocation, ICursor& InCursor);
	/** Stops emiting emitting OnUpdateSliding calls. Unlocks the cursor. Called once either DragSliderCommand or LMB are released. */
	void StopSliding(ICursor& InCursor);
	/** Emits a OnUpdateSliding call. Called once per frame while sliding is active. */
	void UpdateSliding(const FVector2D& InScreenLocation);
};
}

#undef UE_API