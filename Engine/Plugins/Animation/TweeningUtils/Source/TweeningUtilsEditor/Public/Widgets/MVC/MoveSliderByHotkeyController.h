// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Attribute.h"
#include "Templates/PimplPtr.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

#define UE_API TWEENINGUTILSEDITOR_API

class FUICommandInfo;
class FUICommandList;

namespace UE::TweeningUtilsEditor
{
class FTweenModel;
class FTweenMouseSlidingController;

/**
 * Allows the user to move the slider
 * - 100% to the left (FTweeningUtilsCommands::SetSliderLeft)
 * - 100% to the right (FTweeningUtilsCommands::SetSliderRight)
 * while indirectly moving the slider (@see FTweeningUtilsCommands::DragAnimSliderTool).
 */
class FMoveSliderByHotkeyController : public FNoncopyable
{
public:
	
	UE_API explicit FMoveSliderByHotkeyController(TAttribute<FTweenModel*> InTweenModelAttr, const TSharedRef<FUICommandList>& InCommandList);
	UE_API ~FMoveSliderByHotkeyController();
	
	/** @return The current slider position that is being driven by the active command. Range [-1,1]. */
	UE_API TOptional<float> GetCurrentSliderPosition() const;

private:
	
	/** Does the actual blending. */
	const TAttribute<FTweenModel*> TweenModelAttr;
	
	/** The command list that the commands are bound to. */
	const TWeakPtr<FUICommandList> WeakCommandList;
	
	struct FActiveCommandOverride;
	/** Set while one of the commands is pressed down. Becomes unset when the command chord goes up. */
	TPimplPtr<FActiveCommandOverride> ActiveCommand;
	
	/** Handles the command to move the slider to InBlendValue and keeps the slider position there until the key is released. */
	void HandleMoveSlider(float InBlendValue, TWeakPtr<FUICommandInfo> InCommandForChordUp);
	
	/** Handles on of the command chord being released. */
	void HandleCommandChordUp();
};
}

#undef UE_API
