// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API TWEENINGUTILSEDITOR_API

template<typename OptionalType> struct TOptional;

namespace UE::TweeningUtilsEditor
{
class FTweenModel;
class STweenSlider;
enum class ETweenScaleMode : uint8;
	
/**
 * It bridges the STweenSlider, which acts as the view, and the model, FTweenModel, by interchanging data between them.
 * Acts as view in a Model-View-Controller architecture.
 */
class STweenView : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(STweenView) {}
		/** Gets the tween model this view is driving. */
		SLATE_ATTRIBUTE(FTweenModel*, TweenModel)
		
		/** The icon to place in the slider button. */
		SLATE_ATTRIBUTE(const FSlateBrush*, SliderIcon)
		/** The main color. It tints the slider button and the points. */
		SLATE_ATTRIBUTE(FLinearColor, SliderColor)
		/** If set, an indication where to position the slider. Range [-1,1]. If unset, defaults to 0. Ignored if the user is dragging the slider. */
		SLATE_ATTRIBUTE(TOptional<float>, OverrideSliderPosition)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs);

	const TSharedPtr<STweenSlider>& GetTweenSlider() const { return TweenSlider; }

private:

	/** The embedded tween slider this widget is wrapping. */
	TSharedPtr<STweenSlider> TweenSlider;
	
	/** The model that these controls are driving */
	TAttribute<FTweenModel*> TweenModelAttr;
	
	UE_API ETweenScaleMode GetBarRenderMode() const;
	
	UE_API void OnDragStarted() const;
	UE_API void OnDragEnded() const;
	UE_API void OnDragValueUpdated(float Value) const;
	UE_API void OnPointPicked(float Value) const;
	UE_API float MapSliderValueToBlendValue(float Value) const;
};
}

#undef UE_API
