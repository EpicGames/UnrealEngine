// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CycleFunctionController.h"
#include "MoveSliderByHotkeyController.h"
#include "Templates/SharedPointerFwd.h"
#include "TweenMouseSlidingController.h"
#include "TweenToolbarController.h"

#define UE_API TWEENINGUTILSEDITOR_API

class FUICommandList;

namespace UE::TweeningUtilsEditor
{
/** Holds functionality that may be reused across modules to add tweening widgets to a toolbar. */
struct FTweenControllers
{
	/** Manages the toolbar widget */
	FTweenToolbarController ToolbarController;

	/** Cycles functions (Shift + U). */
	FCycleFunctionController CycleFunctionController;

	/** Allows indirect movement of the slider by using u + Move Mouse */
	FTweenMouseSlidingController MouseSlidingController;
	
	/** Allows moving the slider via hotkeys. */
	FMoveSliderByHotkeyController MoveSliderByHotkeyController;

	/**
	 * @param InCommandList The command list to bind commands to
	 * @param InTweenModels Contains all tween models that can be selected by the tweening widgets
	 * @param UserPreferredFunctionContext Optional. The config key under which the selected tween function is saved. The function currently saved under this key will be the first one to be selected.
	 */
	UE_API explicit FTweenControllers(
		const TSharedRef<FUICommandList>& InCommandList,
		const TSharedRef<ITweenModelContainer>& InTweenModels,
		FName UserPreferredFunctionContext = NAME_None
		);
	
	/** @return The current slider override position that may be driven by one of the subsytems, like MouseSlidingController. Range [-1,1]. */
	UE_API TOptional<float> GetCurrentSliderPosition() const;
};
}

#undef UE_API
