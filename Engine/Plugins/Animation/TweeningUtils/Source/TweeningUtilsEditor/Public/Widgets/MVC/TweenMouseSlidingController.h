// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Optional.h"
#include "MouseSlidingController.h"
#include "Templates/SharedPointer.h"

#define UE_API TWEENINGUTILSEDITOR_API

class FUICommandInfo;
class FUICommandList;

namespace UE::TweeningUtilsEditor
{
class FTweenModel;

/** Specialization that sends blend values to a tween model based on how far the mouse is moved. */
class FTweenMouseSlidingController : public FMouseSlidingController
{
public:
	
	/**
	 * Version that allows you to specify every detail
	 * @param InMaxSlideWidthAttr The total size of the "invisible" slider. The mouse can be moved left and right InMaxSlideWidthAttr.Get / 2.0 before being clamped.
	 * @param InTweenModelAttr The tween model that the sliding will update.
	 * @param InCommandList The command list to bind / unbind the command with.
	 * @param InDragSliderCommand The command that triggers detection of mouse movement.
	 */
	UE_API explicit FTweenMouseSlidingController(
		TAttribute<float> InMaxSlideWidthAttr,
		TAttribute<FTweenModel*> InTweenModelAttr,
		const TSharedRef<FUICommandList>& InCommandList,
		TSharedPtr<FUICommandInfo> InDragSliderCommand
		);
	/**
	 * Version that defaults to using the commands from FTweeningUtilsCommands.
	 * The slider width is based on the size of the slider in the toolbar(= value from FTweeningUtilsStyle).
	 */
	UE_API explicit FTweenMouseSlidingController(
		TAttribute<FTweenModel*> InTweenModelAttr,
		const TSharedRef<FUICommandList>& InCommandList
	);

	/** @return The current slider position that is being driven by the mouse. Range [-1,1]. */
	const TOptional<float>& GetCurrentSliderPosition() const { return CurrentSliderPosition; }

private:
	
	/** Does the actual blending. */
	const TAttribute<FTweenModel*> TweenModelAttr;

	/** The current slider position that is being driven by the mouse. Range [-1,1]. */
	TOptional<float> CurrentSliderPosition;

	UE_API void OnSliderStartMove();
	UE_API void OnSliderEndMove();
	UE_API void OnSliderMove(float InValue);
};
}

#undef UE_API
