// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/MultiBox/MultiBoxDefs.h"
#include "Misc/Attribute.h"
#include "Misc/NotNull.h"
#include "Templates/PimplPtr.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

#define UE_API TWEENINGUTILSEDITOR_API

class FText;
class FToolBarBuilder;
class FUICommandList;
class SWidget;
struct FLinearColor;
struct FSlateBrush;
struct FSlateIcon;
template<typename OptionalType> struct TOptional;

namespace UE::TweeningUtilsEditor
{
class FTweenModel;
class ITweenModelContainer;
class STweenSlider;
	
/**
 * Binds together a STweenView and FTweenModel in the context of adding the controls to a toolbar widget.
 * Acts as controller in a Model-View-Controller architecture.
 *
 * Makes a toolbar as follows |Combo|Slider|Overshot|
 * - Combo: Selects the blend function
 * - Slider: Adjusts the blend value and updates the FTweenModel
 * - Overshoot: Toggle button for changing the overshoot property on FTweenModel
 */
class FTweenToolbarController : public FNoncopyable
{
public:
	
	/**
	 * @param InCommandList Used to bind the commands added to the toolbar.
	 * @param InTweenFunctions The functions that can be selected
	 * @param InInitialTweenModelIndex Index of the tween model to initially be selected.
	 */
	UE_API explicit FTweenToolbarController(
		const TSharedRef<FUICommandList>& InCommandList,
		const TSharedRef<ITweenModelContainer>& InTweenFunctions,
		int32 InInitialTweenModelIndex = 0
		);
	UE_API ~FTweenToolbarController();

	struct FMakeWidgetArgs
	{
		/**
		 * Optional override for the where the slider is visually placed (does not affect functionality). Range [-1,1].
		 * Useful e.g. in conjunction with FTweenMouseSlidingController (while pressing U+LMB, give the user a visual association). 
		 */
		TAttribute<TOptional<float>> OverrideSliderPositionAttr;

		/** Affects how the tween function selection combo box is resized. */
		FMenuEntryResizeParams FunctionSelectResizeParams;
		/** Affects how the slider widget is resized. */
		FMenuEntryResizeParams SliderResizeParams;
		/** Affects how the overshoot button is resized. */
		FMenuEntryResizeParams OvershootResizeParams;
	};
	struct FAddToToolbarResult
	{
		/** The tween slider widget that was added to the toolbar. */
		TSharedPtr<STweenSlider> TweenSlider;
	};
	/**
	 * Extends the toolbar. Safe to call multiple times (although typically you'll call this exactly once).
	 * 
	 * @param ToolbarBuilder The toolbar to add the widgets to
	 * @param InArgs Optional additional args to control how the widget should be constructed.
	 */
	UE_API FAddToToolbarResult AddToToolbar(FToolBarBuilder& ToolbarBuilder, FMakeWidgetArgs InArgs = {}) const;

	/** Sets the currently selected tween model. Asserts that the tween model is contained in TweenModels. */
	UE_API void SetSelectedTweenModel(FTweenModel& InTweenModel);

	/** @return Gets the currently selected tween model */
	FTweenModel& GetSelectedTweenModel() const { return *SelectedTweenModel; }
	TAttribute<FTweenModel*> MakeSelectedTweenModelAttr() const { return TAttribute<FTweenModel*>::CreateLambda([this]{ return SelectedTweenModel; }); }
	TAttribute<const FTweenModel*> MakeSelectedConstTweenModelAttr() const { return TAttribute<const FTweenModel*>::CreateLambda([this]{ return SelectedTweenModel; }); }

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnTweenFunctionChanged, const FTweenModel&);
	/** Invoked when the tween function is changed by this controller. */
	FOnTweenFunctionChanged& OnTweenFunctionChanged() { return OnTweenFunctionChangedDelegate; }
	/** Invoked when the overshoot mode is changed by this controller. */
	DECLARE_MULTICAST_DELEGATE(FOnOvershootModeToggled);
	FOnOvershootModeToggled& OnOvershootModeCommandInvoked() { return OnOvershootModeCommandInvokedDelegate; }

private:

	/** Used when the combo button is opened. */
	const TSharedRef<FUICommandList> CommandList;
	/** Holds the functions that can be selected and gives us information about how to display them. */
	const TSharedRef<ITweenModelContainer> TweenModels;
	/** The selected tween model. Points into TweenModels. */
	TNotNull<FTweenModel*> SelectedTweenModel;

	/** Used to override the color of the overshoot button to match the color of the tween model. */
	const TPimplPtr<class FOvershootButtonStyleHack> OverrideStyle;

	/** Invoked when the tween function is changed by this controller. */
	FOnTweenFunctionChanged OnTweenFunctionChangedDelegate;
	/** Invoked when the overshoot mode is changed by this controller. */
	FOnOvershootModeToggled OnOvershootModeCommandInvokedDelegate;

	/** Creates the drop-down list for selecting the tween mode. */
	UE_API TSharedRef<SWidget> MakeTweenModeMenu() const;
	UE_API FText GetLabelForComboBox() const;
	UE_API FText GetToolTipForComboBox() const;
	UE_API FSlateIcon GetIconForComboBox() const;
	
	UE_API void SetTweenModel(FTweenModel* InTweenModel);
	UE_API bool IsTweenModelSelected(FTweenModel* InTweenModel) const;
	
	UE_API void ToggleOvershootMode() const;
	UE_API bool IsOvershootModeEnabled() const;
	UE_API FSlateIcon GetOvershootModeIcon() const;
	
	UE_API const FSlateBrush* GetIconForSlider() const;
	UE_API FLinearColor GetColorForSlider() const;
};
}

#undef UE_API
