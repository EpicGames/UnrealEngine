// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ETweenScaleMode.h"
#include "Styling/SlateWidgetStyleAsset.h" // Required by SLATE_STYLE_ARGUMENT
#include "TweeningUtilsStyle.h"
#include "TweenSliderStyle.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SLeafWidget.h"

#define UE_API TWEENINGUTILSEDITOR_API

template<typename T> struct TOptional;

namespace UE::TweeningUtilsEditor
{
struct FSliderWidgetData;
	
/**
 * The slider ranges from -1 to 1. It drives the blend while pressed down. When released, it jumps to 0: letting the user reapply the blend again.
 
 * The slider looks like this:
 *  - ETweenScaleMode::Normalized: [   +   +   +   [I]   +   +   +   ]
 *  - ETweenScaleMode::Overshoot:  [ + + + | + + + [I] + + + | + + + ]
 * Here:
 *	- I is the Icon and the bit behind it is the SliderIconBackground. Together, they form a button.
 *	- + is a point on the scale.
 */
class STweenSlider : public SLeafWidget
{
public:
	
	DECLARE_DELEGATE_OneParam(FSliderChangedDelegate, float /*Value*/); // Value's range is [-1,1].
	DECLARE_DELEGATE_RetVal_OneParam(float, FMapSliderValueToBlendValue, float /*Value*/); // Value's range is [-1,1].

	SLATE_BEGIN_ARGS(STweenSlider)
		: _Style(&FTweeningUtilsStyle::Get().GetWidgetStyle<FTweenSliderStyle>("TweenSlider"))
		, _ColorAndOpacity(FLinearColor::White)
		, _SliderIcon(nullptr)
		, _SliderColor(FLinearColor(254.f/255.f, 155.f/255.f, 7.f/255.f))
		, _ScaleRenderMode(ETweenScaleMode::Normalized)
		{}
		
		/** The visual style of the button */
		SLATE_STYLE_ARGUMENT( FTweenSliderStyle, Style )

		/** The root opacity of the widget. */
		SLATE_ATTRIBUTE(FSlateColor, ColorAndOpacity)
		
		/** The icon to place in the slider button. */
		SLATE_ATTRIBUTE(const FSlateBrush*, SliderIcon)
		/** The main color. It tints the slider button and the points. */
		SLATE_ATTRIBUTE(FLinearColor, SliderColor)
		/** Affects how the scale is rendered. Events keep returning the -1 to 1 mode - you should compute the scale yourself.*/
		SLATE_ATTRIBUTE(ETweenScaleMode, ScaleRenderMode)

		/** If set, an indication where to position the slider. If unset, defaults to 0. Ignored if the user is dragging the slider. Range [-1,1]. */
		SLATE_ATTRIBUTE(TOptional<float>, OverrideSliderPosition)
		
		/** Invoked when the slider begins being dragged. */
		SLATE_EVENT(FSimpleDelegate, OnSliderDragStarted)
		/** Invoked when the slider stops being dragged */
		SLATE_EVENT(FSimpleDelegate, OnSliderDragEnded)
		/** Invoked for as long as the slider is being dragged. Receives a value from -1 to 1 regardless of ScaleRenderMode. */
		SLATE_EVENT(FSliderChangedDelegate, OnSliderValueDragged)
		/** Invoked when the user presses a point to select a blend value. Receives a value from -1 to 1 regardless of ScaleRenderMode. */
		SLATE_EVENT(FSliderChangedDelegate, OnPointValuePicked)

		/** Invoked to display the correct blend value tooltips. Receives a [-1,1] value and outputs the blend value. */
		SLATE_EVENT(FMapSliderValueToBlendValue, MapSliderValueToBlendValue)
		
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs);

private:

	/**
	 * Abstracts the implementation.
	 *
	 * Specifically, it hides FTweenSliderHoverState, FTweenSliderDrawArgs, and FTweenWidgetArgs so TweenSliderDrawUtils.h does not become part
	 * of the public API.
	 */
	TPimplPtr<FSliderWidgetData> Pimpl;
	
	/** Invoked when the slider begins being dragged. */
	FSimpleDelegate OnSliderDragStartedDelegate;
	/** Invoked when the slider stops being dragged */
	FSimpleDelegate OnSliderDragStoppedDelegate;
	/** Invoked during drag when the slider value changes. */
	FSliderChangedDelegate OnSliderChangedDelegate;
	/** Invoked when the user presses a point to select a blend value. */
	FSliderChangedDelegate OnPointValuePickedDelegate;

	/** Invoked to display the correct blend value tooltips. */
	FMapSliderValueToBlendValue MapSliderValueToBlendValueDelegate;
	
	//~ Begin SLeafWidget Interface
	UE_API virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;
	UE_API virtual FReply OnMouseButtonDown(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent) override;
	UE_API virtual FReply OnMouseButtonUp(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent) override;
	UE_API virtual void OnMouseLeave(const FPointerEvent& InMouseEvent) override;
	UE_API virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& InMouseEvent) override;
	UE_API virtual void OnFinishedPointerInput() override;
	UE_API virtual FVector2D ComputeDesiredSize(float) const override;
	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	UE_API virtual int32 OnPaint(const FPaintArgs& InArgs, const FGeometry& InAllottedGeometry, const FSlateRect& InCullingRect, FSlateWindowElementList& OutDrawElements, int32 InLayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	UE_API virtual TSharedPtr<IToolTip> GetToolTip() override;
	//~ End SLeafWidget Interface

	/** Updates LastMousePositionOnSliderBar. */
	UE_API void UpdateLastMousePositionOnSlider(const FGeometry& InGeometry, const FVector2D& InMousePos);

	UE_API bool IsMouseDown() const;
	UE_API bool IsDragging() const;

	void OnStartDragOp() const;

	/** If the user is currently hovering a point, set the blend value to that point's value.  */
	UE_API void HandlePickValueOfCurrentlyHoveredPoint(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent) const;

	/** Gets the tooltip text for what is currently being hovered. */
	UE_API FText GetToolTipText() const;
};
}

#undef UE_API
