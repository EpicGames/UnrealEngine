// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Layout/Margin.h"
#include "Math/Vector2D.h"
#include "Styling/SlateBrush.h"
#include "Styling/SlateWidgetStyle.h"
#include "TweenSliderStyle.generated.h"

#define UE_API TWEENINGUTILSEDITOR_API

USTRUCT()
struct FTweenPointStyle
{
	GENERATED_BODY()
	
	UE_API FTweenPointStyle();
	UE_API explicit FTweenPointStyle(
		const FVector2D& InNormalSize, const FVector2D& InHoveredSize, const FVector2D& InPressedSize, const FVector2D& InHitSize
		);
	
	/** Brush when a point on the bar is not hovered or pressed */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FSlateBrush Normal;
	FTweenPointStyle& SetNormal( const FSlateBrush& InNormal ){ Normal = InNormal; return *this; }

	/** Brush when a point on the bar is hovered */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FSlateBrush Hovered;
	FTweenPointStyle& SetHovered( const FSlateBrush& InHovered){ Hovered = InHovered; return *this; }

	/** Brush when a point on the bar is pressed */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FSlateBrush Pressed;
	FTweenPointStyle& Set( const FSlateBrush& InPressed ){ Pressed = InPressed; return *this; }

	/** Brush when the slider has passed a point*/
	UPROPERTY(EditAnywhere, Category=Appearance)
	FSlateBrush PassedPoint;
	FTweenPointStyle& SetPassed( const FSlateBrush& InPassed ){ PassedPoint = InPassed; return *this; }
	
	/** The size of the hit box against which hit tests are made for this point (so the user does not need to click them pixel perfectly). */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FVector2D HitTestSize = { 10.0, 12.0 };
	FTweenPointStyle& SetHitTestSize( const FVector2D& InSize ){ HitTestSize = InSize; return *this; }
	
	UE_API void GetResources(TArray<const FSlateBrush*>& OutBrushes) const;
};

/** Style for STweenSlider */
USTRUCT()
struct FTweenSliderStyle : public FSlateWidgetStyle
{
	GENERATED_BODY()

	UE_API FTweenSliderStyle();

	//~ Begin FSlateWidgetStyle Interface
	UE_API virtual void GetResources(TArray<const FSlateBrush*>& OutBrushes) const override;
	//~ End FSlateWidgetStyle Interface
	
	static UE_API const FName TypeName;
	virtual const FName GetTypeName() const override { return TypeName; }
	
	static UE_API const FTweenSliderStyle& GetDefault();
	
	/** The dimensions of the slider. */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FVector2D BarDimensions;
	FTweenSliderStyle& SetBarDimensions( const FVector2D& InDimensions ){ BarDimensions = InDimensions; return *this; }

	/** Brush of the bar on which the points are drawn. The slider brush is drawn over it. Usually the bar's brush is smaller than the button. */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FSlateBrush BarBrush;
	FTweenSliderStyle& SetBarBrush( const FSlateBrush& InBarBrush ){ BarBrush = InBarBrush; return *this; }

	
	
	/** Brush when the button is not hovered or pressed */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FSlateBrush NormalSliderButton;
	FTweenSliderStyle& SetNormalSliderButton( const FSlateBrush& InNormalBrush ){ NormalSliderButton = InNormalBrush; return *this; }

	/** Brush when hovered */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FSlateBrush HoveredSliderButton;
	FTweenSliderStyle& SetHoveredSliderButton( const FSlateBrush& InHoveredBrush){ HoveredSliderButton = InHoveredBrush; return *this; }

	/** Brush when pressed */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FSlateBrush PressedSliderButton;
	FTweenSliderStyle& SetPressedSliderButton( const FSlateBrush& InPressedBrush ){ PressedSliderButton = InPressedBrush; return *this; }


	
	/** Tint for the icon when the button is not hovered or pressed */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FSlateColor NormalIconTint;
	FTweenSliderStyle& SetNormalIconTint( const FSlateColor& InNormalIconTint ){ NormalIconTint = InNormalIconTint; return *this; }

	/** Tint for the icon when hovered */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FSlateColor HoveredIconTint;
	FTweenSliderStyle& SetHoveredIconTint( const FSlateColor& InHoveredIconTint){ HoveredIconTint = InHoveredIconTint; return *this; }

	/** Tint for the icon when pressed */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FSlateColor PressedIconTint;
	FTweenSliderStyle& SetPressedIconTint( const FSlateColor& InPressedIconTint ){ PressedIconTint = InPressedIconTint; return *this; }


	
	/** Brushes for the small points on the bar. */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FTweenPointStyle SmallPoint;
	FTweenSliderStyle& SetSmallPoint( const FTweenPointStyle& InNormal ){ SmallPoint = InNormal; return *this; }

	/** When overshoot mode is enabled, the bars at 100% and -100%. */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FTweenPointStyle MediumPoint;
	FTweenSliderStyle& SetMediumPoint( const FTweenPointStyle& InHovered){ MediumPoint = InHovered; return *this; }

	/** Brushes for points on the left or right end. */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FTweenPointStyle EndPoint;
	FTweenSliderStyle& SetEndPoint( const FTweenPointStyle& InPressed ){ EndPoint = InPressed; return *this; }
	

	
	/** Brush drawn from center to slider when moving the slider (should be a little transparent, drawn over points but under the slider). */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FSlateBrush PassedValueBackground;
	FTweenSliderStyle& SetPassedValueBackground( const FSlateBrush& InBrush ){ PassedValueBackground = InBrush; return *this; }


	
	/** Padding of the icon placed in the slider button */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FMargin IconPadding;
	FTweenSliderStyle& SetIconPadding( const FMargin& InIconPadding ){ IconPadding = InIconPadding; return *this; }
};

#undef UE_API
