// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TweeningUtilsEditor : ModuleRules
{
	public TweeningUtilsEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		bRequiresPlatformSDK = true;
		
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicDependencyModuleNames.AddRange(new string[]
		{
			"Core", 
			"CoreUObject",
			"CurveEditor",
			"Engine",
			"InputCore",
			"SlateCore",
			"Slate",
			"TweeningUtils",
			"UnrealEd"
		});
		
		PrivateDependencyModuleNames.AddRange(new string[]
		{
			
		});
	}
}
