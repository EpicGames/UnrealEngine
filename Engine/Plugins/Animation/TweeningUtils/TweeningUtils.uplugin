{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Tweening Utils",
	"Description": "Algorithms and widgets useful for inbetweening.",
	"Category": "Animation",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": true,
	"CanContainContent": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "TweeningUtils",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "TweeningUtilsEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	]
}